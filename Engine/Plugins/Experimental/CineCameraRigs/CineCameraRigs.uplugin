{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "CineCameraRigs",
	"Description": "Extended camera rigs for cinematic workflow",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CineCameraRigs",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CineCameraRigsEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		}
	]
}
