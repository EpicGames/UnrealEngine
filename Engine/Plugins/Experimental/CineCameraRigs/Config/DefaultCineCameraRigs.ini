[/Script/CineCameraRigs.CineCameraRigRailSettings]
DefaultSplineMeshMaterial=/CineCameraRigs/RigRail/M_RigRail.M_RigRail
DefaultSplineMeshTexture=/CineCameraRigs/RigRail/T_RigRailDefault.T_RigRailDefault
SpeedModeSplineMeshTexture=/CineCameraRigs/RigRail/T_RigRailSpeedMode.T_RigRailSpeedMode