// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CineCameraRigs : ModuleRules
{
	public CineCameraRigs(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CinematicCamera",
				"AnimationCore",
				"Constraints"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"LevelSequence",
				"MovieScene",
				"MovieSceneTracks",
				"Slate",
				"SlateCore",
				"ConcertSyncCore",
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("LevelSequenceEditor");
			PrivateDependencyModuleNames.Add("SequencerScriptingEditor");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

	}
}
