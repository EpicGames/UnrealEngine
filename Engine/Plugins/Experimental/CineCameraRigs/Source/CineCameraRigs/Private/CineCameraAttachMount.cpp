// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraAttachMount.h"

#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"

#include "Components/BillboardComponent.h"
#include "Components/StaticMeshComponent.h"
#include "GameFramework/SpringArmComponent.h"

#include "UObject/ConstructorHelpers.h"
#include "Engine/Texture2D.h"

#if WITH_EDITOR
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "SEditorViewport.h"
#endif

ACineCameraAttachMount::ACineCameraAttachMount(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;

#if WITH_EDITORONLY_DATA
	bShowPreviewMeshes = true;
	PreviewMeshScale = 1.f;
#endif

	USceneComponent* SceneRoot = CreateDefaultSubobject<USceneComponent>("Root");
	SetRootComponent(SceneRoot);

	SpringArmComponent = CreateDefaultSubobject<USpringArmComponent>(TEXT("SpringArm"));
	SpringArmComponent->AttachToComponent(SceneRoot, FAttachmentTransformRules::KeepRelativeTransform);
	SpringArmComponent->bDoCollisionTest = false;
	SpringArmComponent->bEnableCameraLag = true;
	SpringArmComponent->bEnableCameraRotationLag = true;
	SpringArmComponent->TargetArmLength = 0;

#if WITH_EDITOR
	CreatePreviewMesh();
#endif
}

void ACineCameraAttachMount::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
#if WITH_EDITOR
	if (bConstraintUpdated)
	{
		GUnrealEd->UpdatePivotLocationForSelection();
		bConstraintUpdated = false;
	}
	UpdatePreviewMeshes();
#endif
}

bool ACineCameraAttachMount::ShouldTickIfViewportsOnly() const
{
	return true;
}

USceneComponent* ACineCameraAttachMount::GetDefaultAttachComponent() const
{
	return SpringArmComponent;
}

#if WITH_EDITOR
void ACineCameraAttachMount::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, TargetActor) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, TargetSocket))
	{
		CreateConstraint();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, bEnableLocationLag))
	{
		SpringArmComponent->bEnableCameraLag = bEnableLocationLag;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, bEnableRotationLag))
	{
		SpringArmComponent->bEnableCameraRotationLag = bEnableRotationLag;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, LocationLagSpeed))
	{
		SpringArmComponent->CameraLagSpeed = LocationLagSpeed;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, RotationLagSpeed))
	{
		SpringArmComponent->CameraRotationLagSpeed = RotationLagSpeed;
	}
}

void ACineCameraAttachMount::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	const FName StructName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();
	if (StructName == GET_MEMBER_NAME_CHECKED(ACineCameraAttachMount, TransformFilter))
	{
		UpdateAxisFilter();
	}
}
#endif

void ACineCameraAttachMount::UpdateAxisFilter()
{
	if (UTickableParentConstraint* Constraint = GetConstraint())
	{
		Constraint->TransformFilter = TransformFilter;
	}
}

void ACineCameraAttachMount::CreateConstraint()
{
	UWorld* World = GetWorld();
	TArray< TWeakObjectPtr<UTickableConstraint> > Constraints;
	UE::TransformConstraintUtil::GetParentConstraints(World, this, Constraints);

	for (const TWeakObjectPtr<UTickableConstraint>& Constraint : Constraints)
	{
		FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.RemoveConstraint(Constraint.Get());
	}

	if (!TargetActor.IsValid())
	{
		return;
	}

	UTickableTransformConstraint* Constraint = UE::TransformConstraintUtil::CreateAndAddFromObjects(World, TargetActor.Get(), TargetSocket, this, NAME_None, ETransformConstraintType::Parent);
	if (Constraint)
	{
		Constraint->bDynamicOffset = true;
		Cast<UTickableParentConstraint>(Constraint)->OffsetTransform = FTransform::Identity;
		FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.StaticConstraintCreated(Constraint);
	}
	bConstraintUpdated = true;
}

UTickableParentConstraint* ACineCameraAttachMount::GetConstraint()
{
	UWorld* World = GetWorld();
	TArray< TWeakObjectPtr<UTickableConstraint> > Constraints;
	UE::TransformConstraintUtil::GetParentConstraints(World, this, Constraints);
	if (Constraints.Num() == 0)
	{
		return nullptr;
	}
	return Cast<UTickableParentConstraint>(Constraints[0].Get());
}

void ACineCameraAttachMount::ResetLocationOffset()
{
	if (UTickableParentConstraint* Constraint = GetConstraint())
	{
		Constraint->OffsetTransform.SetLocation(FVector::ZeroVector);
		bConstraintUpdated = true;
	}
}

void ACineCameraAttachMount::ResetRotationOffset()
{
	if (UTickableParentConstraint* Constraint = GetConstraint())
	{
		Constraint->OffsetTransform.SetRotation(FQuat::Identity);
		bConstraintUpdated = true;
	}
}

void ACineCameraAttachMount::ZeroRoll()
{
	if (UTickableParentConstraint* Constraint = GetConstraint())
	{
		FTransform ParentTransform = Constraint->GetParentGlobalTransform();
		FTransform ChildTransform = Constraint->GetChildGlobalTransform();
		FRotator WorldRot = ChildTransform.Rotator();
		WorldRot.Roll = 0;
		ChildTransform.SetRotation(FQuat(WorldRot));
		Constraint->OffsetTransform = ChildTransform.GetRelativeTransform(ParentTransform);
		bConstraintUpdated = true;
	}
	else
	{
		FRotator WorldRot = RootComponent->GetComponentRotation();
		WorldRot.Roll = 0;
		RootComponent->SetWorldRotation(WorldRot);
	}
}

void ACineCameraAttachMount::ZeroTilt()
{
	if (UTickableParentConstraint* Constraint = GetConstraint())
	{
		FTransform ParentTransform = Constraint->GetParentGlobalTransform();
		FTransform ChildTransform = Constraint->GetChildGlobalTransform();
		FRotator WorldRot = ChildTransform.Rotator();
		WorldRot.Pitch = 0;
		ChildTransform.SetRotation(FQuat(WorldRot));
		Constraint->OffsetTransform = ChildTransform.GetRelativeTransform(ParentTransform);
		bConstraintUpdated = true;
	}
	else
	{
		FRotator WorldRot = RootComponent->GetComponentRotation();
		WorldRot.Pitch = 0;
		RootComponent->SetWorldRotation(WorldRot);
	}
}

void ACineCameraAttachMount::SetEnableLocationLag(bool bEnabled)
{
	SpringArmComponent->bEnableCameraLag = bEnableLocationLag = bEnabled;
}

void ACineCameraAttachMount::SetEnableRotationLag(bool bEnabled)
{
	SpringArmComponent->bEnableCameraRotationLag = bEnableRotationLag = bEnabled;
}

void ACineCameraAttachMount::SetLocationLagSpeed(float Speed)
{
	SpringArmComponent->CameraLagSpeed = LocationLagSpeed = Speed;
}

void ACineCameraAttachMount::SetRotationLagSpeed(float Speed)
{
	SpringArmComponent->CameraRotationLagSpeed = RotationLagSpeed = Speed;
}

void ACineCameraAttachMount::SetTransformFilter(const FTransformFilter& InFilter)
{
	TransformFilter = InFilter;
	UpdateAxisFilter();
}

#if WITH_EDITOR

void ACineCameraAttachMount::CreatePreviewMesh()
{
	static ConstructorHelpers::FObjectFinder<UStaticMesh> PreviewMesh(TEXT("/CineCameraRigs/AttachMount/SM_CineCameraAttachMount_PreviewMesh.SM_CineCameraAttachMount_PreviewMesh"));
	PreviewMesh_Root = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("PreviewMesh_Root"));
	if (PreviewMesh_Root)
	{
		PreviewMesh_Root->SetStaticMesh(PreviewMesh.Object);
		PreviewMesh_Root->SetIsVisualizationComponent(true);
		PreviewMesh_Root->SetVisibility(bShowPreviewMeshes);
		PreviewMesh_Root->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		PreviewMesh_Root->bHiddenInGame = true;
		PreviewMesh_Root->CastShadow = false;
		PreviewMesh_Root->SetWorldScale3D(FVector(PreviewMeshScale, PreviewMeshScale, PreviewMeshScale));
		PreviewMesh_Root->SetupAttachment(RootComponent);
	}

	static ConstructorHelpers::FObjectFinder<UStaticMesh> MountMeshData(TEXT("/Engine/EditorMeshes/Camera/SM_RailRig_Mount.SM_RailRig_Mount"));
	MountMesh = MountMeshData.Object;
}

void ACineCameraAttachMount::UpdatePreviewMeshes()
{
	if (PreviewMesh_Root)
	{
		PreviewMesh_Root->SetVisibility(bShowPreviewMeshes);
		PreviewMesh_Root->SetWorldScale3D(FVector(PreviewMeshScale, PreviewMeshScale, PreviewMeshScale));
	}

	TArray< AActor* > AttachedActors;
	GetAttachedActors(AttachedActors);

	if (PreviewMeshes_Mount.Num() != AttachedActors.Num())
	{
		for (UStaticMeshComponent* PreviewMesh : PreviewMeshes_Mount)
		{
			PreviewMesh->UnregisterComponent();
			PreviewMesh->Modify();
			PreviewMesh->DestroyComponent();
		}
		PreviewMeshes_Mount.Empty();

		for (int Index = 0; Index < AttachedActors.Num(); ++Index)
		{
			UStaticMeshComponent* const PreviewMesh = NewObject<UStaticMeshComponent>(this);
			if (PreviewMesh)
			{
				PreviewMesh->SetStaticMesh(MountMesh);
				PreviewMesh->SetIsVisualizationComponent(true);
				PreviewMesh->SetVisibility(bShowPreviewMeshes);
				PreviewMesh->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				PreviewMesh->bHiddenInGame = true;
				PreviewMesh->CastShadow = false;
				PreviewMesh->SetWorldScale3D(FVector(PreviewMeshScale, PreviewMeshScale, PreviewMeshScale));
				PreviewMesh->SetupAttachment(RootComponent);
				PreviewMesh->RegisterComponent();

				PreviewMeshes_Mount.Add(PreviewMesh);
			}
		}

	}
	check(PreviewMeshes_Mount.Num() == AttachedActors.Num());

	for (int Index = 0; Index < PreviewMeshes_Mount.Num(); ++Index)
	{
		const FTransform TargetXform = AttachedActors[Index]->GetActorTransform();
		PreviewMeshes_Mount[Index]->SetWorldLocation(TargetXform.GetLocation());
		PreviewMeshes_Mount[Index]->SetVisibility(bShowPreviewMeshes);
		PreviewMeshes_Mount[Index]->SetWorldScale3D(FVector(PreviewMeshScale, PreviewMeshScale, PreviewMeshScale));
	}

}
#endif 