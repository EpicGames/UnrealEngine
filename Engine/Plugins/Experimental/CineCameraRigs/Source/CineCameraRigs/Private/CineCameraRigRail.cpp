// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraRigRail.h"
#include "CineCameraRigRailHelpers.h"
#include "CineCameraRigsSettings.h"

#include "CineCameraComponent.h"
#include "CineSplineComponent.h"
#include "Components/SplineMeshComponent.h"

#include "Algo/AnyOf.h"
#include "Engine/World.h"
#include "Kismet/KismetMathLibrary.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "SequencerTools.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#endif

ACineCameraRigRail::ACineCameraRigRail(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.SetDefaultSubobjectClass<UCineSplineComponent>(TEXT("RailSplineComponent"))
	)
{
	CineSplineComponent = Cast<UCineSplineComponent>(GetRailSplineComponent());
	if (CineSplineComponent != nullptr)
	{
		CineSplineComponent->OnSplineEdited.AddUObject(this, &ACineCameraRigRail::OnSplineEdited);
		CineSplineComponent->Duration = 10.0f;
	}

	UMaterialInterface* DefaultMaterial = GetDefault<UCineCameraRigRailSettings>()->DefaultSplineMeshMaterial.LoadSynchronous();
	if (DefaultMaterial)
	{
		SplineMeshMaterial = DefaultMaterial;
	}

	UTexture2D* DefaultTexture = GetDefault<UCineCameraRigRailSettings>()->DefaultSplineMeshTexture.LoadSynchronous();
	if (DefaultTexture)
	{
		SplineMeshTexture = DefaultTexture;
	}

	bLockOrientationToRail = true;
	SetActorHiddenInGame(true);

#if WITH_EDITORONLY_DATA
	if (PreviewMesh_Mount)
	{
		PreviewMesh_Mount->bHiddenInGame = false;
	}
#endif

#if WITH_EDITOR
	if (!IsInAsyncLoadingThread())
	{
		ConditionalRegisterTimerManager();
	}
#endif
}

#if WITH_EDITOR
void ACineCameraRigRail::ConditionalRegisterTimerManager()
{
	if (!SequencerCheckHandle.IsValid() && !IsTemplate() && GEditor)
	{
		GEditor->GetTimerManager()->SetTimer(SequencerCheckHandle, this, &ACineCameraRigRail::OnSequencerCheck, 1.0, true);
	}
}
#endif

void ACineCameraRigRail::UpdateRailComponents()
{
	if (CineSplineComponent == nullptr)
	{
		return;
	}

	if (!GetWorld())
	{
		return;
	}


	FVector SplinePosition;
	FQuat SplineQuat;
	float InputKey = 0.0f;

	if (bUseAbsolutePosition)
	{
		InputKey = CineSplineComponent->GetInputKeyAtPosition(AbsolutePositionOnRail);
		SplinePosition = CineSplineComponent->GetLocationAtSplineInputKey(InputKey, ESplineCoordinateSpace::World);
		SplineQuat = CineSplineComponent->GetQuaternionAtSplineInputKey(InputKey, ESplineCoordinateSpace::World);
	}
	else
	{
		float const SplineLen = CineSplineComponent->GetSplineLength();
		SplinePosition = CineSplineComponent->GetLocationAtDistanceAlongSpline(CurrentPositionOnRail * SplineLen, ESplineCoordinateSpace::World);
		SplineQuat = CineSplineComponent->GetQuaternionAtDistanceAlongSpline(CurrentPositionOnRail * SplineLen, ESplineCoordinateSpace::World);
		InputKey = CineSplineComponent->GetInputKeyValueAtDistanceAlongSpline(CurrentPositionOnRail * SplineLen);
	}

	if (bUsePointRotation)
	{
		SplineQuat = CineSplineComponent->GetComponentTransform().GetRotation() * CineSplineComponent->GetPointRotationAtSplineInputKey(InputKey);
	}

	if (USceneComponent* AttachComponent = GetDefaultAttachComponent())
	{
		FVector Position = GetActorLocation();
		FRotator Rotation = GetActorRotation();

		if (bAttachLocationX)
		{
			Position.X = SplinePosition.X;
		}
		if (bAttachLocationY)
		{
			Position.Y = SplinePosition.Y;
		}
		if (bAttachLocationZ)
		{
			Position.Z = SplinePosition.Z;
		}
		if (bAttachRotationX && bLockOrientationToRail)
		{
			Rotation.Roll = SplineQuat.Rotator().Roll;
		}
		if (bAttachRotationY && bLockOrientationToRail)
		{
			Rotation.Pitch = SplineQuat.Rotator().Pitch;
		}
		if (bAttachRotationZ && bLockOrientationToRail)
		{
			Rotation.Yaw = SplineQuat.Rotator().Yaw;
		}
		AttachComponent->SetWorldTransform(FTransform(Rotation, Position));
	}

	TArray< AActor* > AttachedActors;
	GetAttachedActors(AttachedActors);

	float FocalLengthValue = CineSplineComponent->GetFloatPropertyAtSplineInputKey(InputKey, FName("FocalLength"));
	float ApertureValue = CineSplineComponent->GetFloatPropertyAtSplineInputKey(InputKey, FName("Aperture"));
	float FocusDistanceValue = CineSplineComponent->GetFloatPropertyAtSplineInputKey(InputKey, FName("FocusDistance"));
	for (AActor* AttachedActor : AttachedActors)
	{
		if (UCineCameraComponent* CameraComponent = Cast<UCineCameraComponent>(AttachedActor->GetComponentByClass(UCineCameraComponent::StaticClass())))
		{
			if (bInheritFocusDistance)
			{
				CameraComponent->FocusSettings.ManualFocusDistance = FocusDistanceValue;
			}
			if (bInheritFocalLength)
			{
				CameraComponent->SetCurrentFocalLength(FocalLengthValue);
			}
			if (bInheritAperture)
			{
				CameraComponent->SetCurrentAperture(ApertureValue);
			}
		}
	}
#if WITH_EDITOR
	if (GIsEditor)
	{
		const UWorld* const MyWorld = GetWorld();
		if (MyWorld && !MyWorld->IsGameWorld())
		{
			UpdatePreviewMeshes();
		}

		// Set HiddenInGame false on the spline mesh so that it can show up in game mode
		// It can still be hidden via actor's HiddenInGame property
		// Also update material if numbers don't match
		int32 NumSegments = PreviewRailMeshSegments.Num();
		if (NumSegments != SplineMeshMIDs.Num())
		{
			UpdateSplineMeshMID();
		}
		for (int Index = 0; Index < NumSegments; ++Index)
		{
			USplineMeshComponent* SplineMeshComp = PreviewRailMeshSegments[Index];
			if (SplineMeshComp)
			{
				SplineMeshComp->bHiddenInGame = false;
				if (SplineMeshMIDs.Num() > Index && IsValid(SplineMeshMIDs[Index]))
				{
					SplineMeshComp->SetMaterial(0, SplineMeshMIDs[Index]);
				}
			}
		}
	}
#endif
}

void ACineCameraRigRail::UpdateSplineMeshMID()
{
	if (GetWorld())
	{
		if (IsValid(SplineMeshMaterial))
		{
			int32 const NumSplinePoints = RailSplineComponent->GetNumberOfSplinePoints();
			int32 const NumNeededMIDs = FMath::Max(0, NumSplinePoints - 1);
			if (SplineMeshMIDs.Num() > NumNeededMIDs)
			{
				int32 const NumToRemove = SplineMeshMIDs.Num() - NumNeededMIDs;
				for (int32 Index = 0; Index < NumToRemove; ++Index)
				{
					SplineMeshMIDs.Pop();
				}
			}
			else
			{
				int32 const NumToAdd = NumNeededMIDs - SplineMeshMIDs.Num();
				for (int Index = 0; Index < NumToAdd; ++Index)
				{
					UMaterialInstanceDynamic* SplineMeshMID = UMaterialInstanceDynamic::Create(SplineMeshMaterial, nullptr, MakeUniqueObjectName(GetTransientPackage(), UMaterialInstanceDynamic::StaticClass(), TEXT("SplineMeshMID")));
					SplineMeshMID->SetFlags(RF_Transient | RF_NonPIEDuplicateTransient | RF_TextExportTransient);
					SplineMeshMIDs.Add(SplineMeshMID);
				}
			}
			
			for(int32 Index = 0; Index < NumNeededMIDs; ++Index)
			{
				UMaterialInstanceDynamic* SplineMeshMID = SplineMeshMIDs[Index];
				if (!IsValid(SplineMeshMID) || (SplineMeshMID->Parent != SplineMeshMaterial))
				{
					SplineMeshMID = UMaterialInstanceDynamic::Create(SplineMeshMaterial, nullptr, MakeUniqueObjectName(GetTransientPackage(), UMaterialInstanceDynamic::StaticClass(), TEXT("SplineMeshMID")));
					SplineMeshMID->SetFlags(RF_Transient | RF_NonPIEDuplicateTransient | RF_TextExportTransient);
					SplineMeshMIDs[Index] = SplineMeshMID;
				}
			}
		}
	}
}

#if WITH_EDITOR
void ACineCameraRigRail::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, SplineMeshMaterial) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, SplineMeshTexture))
	{
			UpdateSplineMeshMID();
			SetMIDParameters();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, AbsolutePositionOnRail) ||
			 PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, CurrentPositionOnRail))
	{
		UpdateSpeedProgress();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, DriveMode))
	{
		UpdateSpeedProgress();
		UpdateSpeedHeatmap();
		UpdateSplineMeshMID();
		SetMIDParameters();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bDisplaySpeedHeatmap) ||
		     PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, SpeedSampleCountPerSegment) || 
			 PropertyName == GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bUseAbsolutePosition))
	{
		UpdateSpeedHeatmap();
		UpdateSplineMeshMID();
		SetMIDParameters();
	}

}
#endif

void ACineCameraRigRail::SetSplineMeshMaterial(UMaterialInterface* InMaterial)
{
	SplineMeshMaterial = InMaterial;
	UpdateSplineMeshMID();
	SetMIDParameters();
}
void ACineCameraRigRail::SetSplineMeshTexture(UTexture2D* InTexture)
{
	SplineMeshTexture = InTexture;
	UpdateSplineMeshMID();
	SetMIDParameters();
}

void ACineCameraRigRail::SetAbsolutePositionOnRail(float Value)
{
	AbsolutePositionOnRail = Value;
	UpdateSpeedProgress();
}

FVector ACineCameraRigRail::GetVelocityAtPosition(const float InPosition, const float delta) const
{
	if (CineSplineComponent->Duration == 0.0f)
	{
		return FVector::ZeroVector;
	}

	float TimeMultiplier = 1.0f / CineSplineComponent->Duration;

	if (bUseAbsolutePosition)
	{
		int32 const NumPoints = CineSplineComponent->GetNumberOfSplinePoints();
		float const MinPosition = CineSplineComponent->GetFloatPropertyAtSplinePoint(0, FName(TEXT("AbsolutePosition")));
		float const MaxPosition = CineSplineComponent->GetFloatPropertyAtSplinePoint(NumPoints - 1, FName(TEXT("AbsolutePosition")));
		float const t0 = InPosition + delta >= MaxPosition ? InPosition - delta : InPosition;
		float const t1 = InPosition + delta >= MaxPosition ? InPosition : InPosition + delta;
		float const InKey0 = CineSplineComponent->GetInputKeyAtPosition(t0);
		float const InKey1 = CineSplineComponent->GetInputKeyAtPosition(t1);
		FVector const P0 = CineSplineComponent->GetLocationAtSplineInputKey(InKey0, ESplineCoordinateSpace::World);
		FVector const P1 = CineSplineComponent->GetLocationAtSplineInputKey(InKey1, ESplineCoordinateSpace::World);
		return (P1 - P0) / (t1 - t0) * TimeMultiplier * (MaxPosition - MinPosition);
	}
	else
	{
		float const SplineLen = CineSplineComponent->GetSplineLength();
		float const t0 = InPosition + delta >= 1.0f ? InPosition - delta : InPosition;
		float const t1 = InPosition + delta >= 1.0f ? InPosition : InPosition + delta;
		FVector const P0 = CineSplineComponent->GetLocationAtDistanceAlongSpline(t0 * SplineLen, ESplineCoordinateSpace::World);
		FVector const P1 = CineSplineComponent->GetLocationAtDistanceAlongSpline(t1 * SplineLen, ESplineCoordinateSpace::World);
		return (P1 - P0) / (t1 - t0) * TimeMultiplier;
	}
}

void ACineCameraRigRail::OnSplineEdited()
{
	UpdateSpeedHeatmap();
	UpdateSplineMeshMID();
	SetMIDParameters();
}

void ACineCameraRigRail::SetMIDParameters()
{
	int32 NumMIDs = SplineMeshMIDs.Num();
	for (int32 Index = 0; Index < NumMIDs; ++Index)
	{
		UMaterialInstanceDynamic* MID = SplineMeshMIDs[Index];
		MID->SetScalarParameterValue(TEXT("SplineParamStart"), UKismetMathLibrary::SafeDivide((float)Index, (float)NumMIDs));
		MID->SetScalarParameterValue(TEXT("SplineParamEnd"), UKismetMathLibrary::SafeDivide((float)Index+1, (float)NumMIDs));
		if (IsValid(SplineMeshTexture))
		{
			MID->SetTextureParameterValue(TEXT("SplineTexture"), SplineMeshTexture);
		}
	}
}

void ACineCameraRigRail::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	if (DriveMode == ECineCameraRigRailDriveMode::Duration && bPlay)
	{
		DriveByParam(DeltaTime);
	}
	else if (DriveMode == ECineCameraRigRailDriveMode::Speed && bPlay)
	{
		DriveBySpeed(DeltaTime);
	}
}
void ACineCameraRigRail::PostLoad()
{
	Super::PostLoad();
	if (CineSplineComponent)
	{
		UpdateSpeedHeatmap();
		UpdateSplineMeshMID();
		SetMIDParameters();
	}

#if WITH_EDITOR
	ConditionalRegisterTimerManager();
#endif
}

void ACineCameraRigRail::DriveByParam(float DeltaTime)
{
	const float AdjustedDeltaTime = AdjustDeltaTime(DeltaTime);
	const float PositionDuration = LastPositionValue() - StartPositionValue();
	const float Increment = PlayForward() ? AdjustedDeltaTime : -AdjustedDeltaTime;
	float Param = PositionDuration * UKismetMathLibrary::SafeDivide(Increment, CineSplineComponent->Duration);
	Param += bUseAbsolutePosition ? AbsolutePositionOnRail : CurrentPositionOnRail;

	// Reverse direction in Bounce mode when Param goes outside of the range 
	if (LoopMode == ECineCameraRigRailLoopMode::Bounce)
	{
		if (Param < StartPositionValue() || Param > LastPositionValue())
		{
			bBounceForward = !bBounceForward;
		}
	}

	// Mod or Clamp the Param variable so that it's within the range
	if (LoopMode == ECineCameraRigRailLoopMode::Loop)
	{
		Param = PositionDuration > 0.0f ? FMath::Fmod(Param - StartPositionValue() + PositionDuration, PositionDuration) : 0.0;
		Param += StartPositionValue();
	}
	else
	{
		Param = FMath::Clamp(Param, StartPositionValue(), LastPositionValue());
	}
	bUseAbsolutePosition ? AbsolutePositionOnRail = Param : CurrentPositionOnRail = Param;
}

void ACineCameraRigRail::DriveBySpeed(float DeltaTime)
{
	if (Speed == 0.0f)
	{
		return;
	}
	const float AdjustedDeltaTime = AdjustDeltaTime(DeltaTime);
	const float TotalTime = CineSplineComponent->GetSplineLength() / FMath::Abs(Speed);
	float CurrentTime = TotalTime * SpeedProgress;
	const float Increment = PlayForward() ? AdjustedDeltaTime : -AdjustedDeltaTime;
	CurrentTime += (FMath::Sign(Speed) * Increment);

	// Reverse direction in Bounce mode when CurrenTime goes outside of the range
	if (LoopMode == ECineCameraRigRailLoopMode::Bounce)
	{
		if (CurrentTime < 0.0f || CurrentTime > TotalTime)
		{
			bBounceForward = !bBounceForward;
		}
	}

	// Mod or Clamp the CurrentTime variable so that it's within the range
	CurrentTime = LoopMode == ECineCameraRigRailLoopMode::Loop ? FMath::Fmod(CurrentTime + TotalTime, TotalTime) : FMath::Clamp(CurrentTime, 0.0f, TotalTime);

	SpeedProgress = CurrentTime / TotalTime;
	
	if (bUseAbsolutePosition)
	{
		const float Distance = CineSplineComponent->GetSplineLength() * SpeedProgress;
		const float InputKey = CineSplineComponent->GetInputKeyValueAtDistanceAlongSpline(Distance);
		AbsolutePositionOnRail = CineSplineComponent->GetPositionAtInputKey(InputKey);
	}
	else
	{
		CurrentPositionOnRail = SpeedProgress;
	}
}

float ACineCameraRigRail::StartPositionValue() const
{
	return bUseAbsolutePosition ? CineSplineComponent->GetFloatPropertyAtSplinePoint(0, FName(TEXT("AbsolutePosition"))) : 0.0f;
}

float ACineCameraRigRail::LastPositionValue() const
{
	int32 const NumPoints = CineSplineComponent->GetNumberOfSplinePoints();
	return bUseAbsolutePosition ? CineSplineComponent->GetFloatPropertyAtSplinePoint(NumPoints - 1, FName(TEXT("AbsolutePosition"))) : 1.0f;
}

void ACineCameraRigRail::SetDriveMode(ECineCameraRigRailDriveMode InMode)
{
	DriveMode = InMode;
	UpdateSpeedProgress();
	UpdateSpeedHeatmap();
	UpdateSplineMeshMID();
	SetMIDParameters();
}

void ACineCameraRigRail::SetDisplaySpeedHeatmap(bool bEnable)
{
	bDisplaySpeedHeatmap = bEnable;
	UpdateSpeedHeatmap();
	UpdateSplineMeshMID();
	SetMIDParameters();
}

void ACineCameraRigRail::UpdateSpeedProgress()
{
	const float Length = CineSplineComponent->GetSplineLength();
	if (bUseAbsolutePosition)
	{
		const float InputKey = CineSplineComponent->GetInputKeyAtPosition(AbsolutePositionOnRail);
		const float Distance = CineSplineComponent->GetDistanceAlongSplineAtSplineInputKey(InputKey);
		SpeedProgress = UKismetMathLibrary::SafeDivide(Distance, Length);
	}
	else
	{
		SpeedProgress = CurrentPositionOnRail;
	}
}

void ACineCameraRigRail::UpdateSpeedHeatmap()
{
	// If bDisplaySpeedHeatmap is false, assigns default grey texture
	// If DriveMode is Speed, assigns green texture indicating it's constant speed
	// Otherwise, samples the velocity along the spline and generates a heatmap texture

	if (!bDisplaySpeedHeatmap || IsSequencerDriven())
	{
		UTexture2D* DefaultTexture = GetDefault<UCineCameraRigRailSettings>()->DefaultSplineMeshTexture.LoadSynchronous();
		if (DefaultTexture)
		{
			SplineMeshTexture = DefaultTexture;
		}
		return;
	}
	else if (DriveMode == ECineCameraRigRailDriveMode::Speed)
	{
		UTexture2D* SpeedModeTexture = GetDefault<UCineCameraRigRailSettings>()->SpeedModeSplineMeshTexture.LoadSynchronous();
		if (SpeedModeTexture)
		{
			SplineMeshTexture = SpeedModeTexture;
		}
		return;
	}

	TArray<float> SpeedValues;
	const int32 NumPoints = CineSplineComponent->GetNumberOfSplinePoints();
	const int32 NumSegments = CineSplineComponent->GetNumberOfSplineSegments();

	SpeedValues.SetNum(NumSegments * SpeedSampleCountPerSegment + 1);
	const float SplineLen = CineSplineComponent->GetSplineLength();

	for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		for (int32 PerSegmentIndex = 0; PerSegmentIndex < SpeedSampleCountPerSegment; ++PerSegmentIndex)
		{
			int32 Index = SegmentIndex * SpeedSampleCountPerSegment + PerSegmentIndex;
			const float InputKey = (float)SegmentIndex + (float)PerSegmentIndex / (float)SpeedSampleCountPerSegment;
			if (bUseAbsolutePosition)
			{
				const float Position = CineSplineComponent->GetPositionAtInputKey(InputKey);
				SpeedValues[Index] = GetVelocityAtPosition(Position).Length();
			}
			else
			{
				const float Distance = CineSplineComponent->GetDistanceAlongSplineAtSplineInputKey(InputKey);
				SpeedValues[Index] = GetVelocityAtPosition(UKismetMathLibrary::SafeDivide(Distance, SplineLen)).Length();
			}
		}
	}
	// Last Point
	const int32 Index = NumSegments * SpeedSampleCountPerSegment;
	const float InputKey = (float)NumPoints;
	if (bUseAbsolutePosition)
	{
		const float Position = CineSplineComponent->GetPositionAtInputKey(InputKey);
		SpeedValues[Index] = GetVelocityAtPosition(Position).Length();
	}
	else
	{
		const float Distance = CineSplineComponent->GetDistanceAlongSplineAtSplineInputKey(InputKey);
		SpeedValues[Index] = GetVelocityAtPosition(UKismetMathLibrary::SafeDivide(Distance, SplineLen)).Length();
	}

	float AverageValue = 0.0f;
	for (float SpeedValue : SpeedValues)
	{
		AverageValue += SpeedValue;
	}
	AverageValue /= (float)SpeedValues.Num();

	float LowValue = AverageValue * 0.5f;
	float HighValue = AverageValue * 2.0f;
	UCineCameraRigRailHelpers::CreateOrUpdateSplineHeatmapTexture(static_cast<UTexture2D*&>(SplineMeshTexture), SpeedValues, LowValue, AverageValue, HighValue);
}

bool ACineCameraRigRail::IsSequencerDriven()
{
#if WITH_EDITOR
	return bSequencerDriven;
#else
	return false;
#endif
}

float ACineCameraRigRail::AdjustDeltaTime(float InDeltaTime) const
{
	const float TimeDilation = FMath::Max(GetActorTimeDilation(), UE_KINDA_SMALL_NUMBER);
	return bCompensateTimeScale ? InDeltaTime / TimeDilation : InDeltaTime;
}

bool ACineCameraRigRail::PlayForward() const
{
	if (LoopMode == ECineCameraRigRailLoopMode::Bounce)
	{
		return bBounceForward;
	}
	return !bReverse;
}

#if WITH_EDITOR
UMovieSceneFloatTrack* ACineCameraRigRail::FindPositionTrack(const UMovieSceneSequence* InSequence)
{
	if (!IsValid(InSequence) || !InSequence->GetMovieScene())
	{
		return nullptr;
	}
	
	const UMovieScene* MovieScene = InSequence->GetMovieScene();

	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
	const FName PropertyName = bUseAbsolutePosition ? FName(TEXT("AbsolutePositionOnRail")) : FName(TEXT("CurrentPositionOnRail"));
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UObject*, TInlineAllocator<1>> BoundObjects;
		InSequence->LocateBoundObjects(Binding.GetObjectGuid(), UE::UniversalObjectLocator::FResolveParams(GetWorld()), MovieSceneHelpers::CreateTransientSharedPlaybackState(GetWorld(), const_cast<UMovieSceneSequence*>(InSequence)), BoundObjects);
		if (BoundObjects.IsEmpty() || BoundObjects[0] != this)
		{
			continue;
		}
		if (UMovieSceneFloatTrack* Track = MovieScene->FindTrack<UMovieSceneFloatTrack>(Binding.GetObjectGuid(), PropertyName))
		{
			return Track;
		}
	}

	// Find in subsequences
	for (const UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		const UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
		if (!SubTrack || SubTrack->IsEvalDisabled())
		{
			continue;
		}

		for( const UMovieSceneSection* Section : SubTrack->GetAllSections())
		{
			const UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
			if (!SubSection)
			{
				continue;
			}

			UMovieSceneSequence* SubSequence = SubSection->GetSequence();
			if (!SubSequence)
			{
				continue;
			}

			if (UMovieSceneFloatTrack* SubSequenceTrack = FindPositionTrack(SubSequence))
			{
				return SubSequenceTrack;
			}
		}
	}

	return nullptr;
}

void ACineCameraRigRail::OnSequencerCheck()
{
	if (!GetWorld())
	{
		return;
	}

	bool bHasKey = false;
	ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	UMovieSceneFloatTrack* Track = FindPositionTrack(LevelSequence);
	if (IsValid(Track) && !Track->IsEvalDisabled())
	{
		bHasKey = Algo::AnyOf(Track->GetAllSections(), [](const UMovieSceneSection* Section)
		{
			const FMovieSceneFloatChannel* Channel = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>()[0];
			return Channel->GetNumKeys() > 0;
		});
	}

	if(bSequencerDriven != bHasKey)
	{
		bSequencerDriven = !bSequencerDriven;
		UpdateSpeedHeatmap();
		UpdateSplineMeshMID();
		SetMIDParameters();
	}
}
#endif
