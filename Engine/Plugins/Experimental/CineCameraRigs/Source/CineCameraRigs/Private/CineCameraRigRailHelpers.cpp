// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraRigRailHelpers.h"
#include "CineCameraRigRail.h"

#include "Kismet/KismetMathLibrary.h"
#include "Engine/TextureDefines.h"
#include "Engine/Texture.h"
#include "TextureResource.h"

void UCineCameraRigRailHelpers::CreateOrUpdateSplineHeatmapTexture(UTexture2D*& Texture, const TArray<float>& DataValues, const float LowValue, const float MidValue, const float HighValue)
{
#if WITH_EDITOR
	int32 Width = DataValues.Num();
	int32 Height = 1;
	TArray<FColor> Pixels;
	Pixels.Init(FColor::Black, Height * Width);
	float HighRange = HighValue - MidValue;
	for (int32 Index = 0; Index < Width; ++Index)
	{
		float UpperWeight = UKismetMathLibrary::SafeDivide(FMath::Clamp(DataValues[Index] - MidValue, 0.0f, HighValue - MidValue), HighValue - MidValue);
		float LowerWeight = UKismetMathLibrary::SafeDivide(FMath::Clamp(MidValue - DataValues[Index], 0.0f, MidValue - LowValue), MidValue - LowValue);
		Pixels[Index].R = (uint8)(UpperWeight * 255);
		Pixels[Index].G = (uint8)((1.0 - (UpperWeight + LowerWeight)) * 255);
		Pixels[Index].B = (uint8)(LowerWeight * 255);
		Pixels[Index].A = DataValues[Index] >= LowValue ? 255 : 0;
	}

	if (Texture == nullptr || Texture->GetSizeX() != Width || Texture->GetSizeY() != Height)
	{
		Texture = UTexture2D::CreateTransient(Width, Height, EPixelFormat::PF_B8G8R8A8);
		Texture->SRGB = false;
		Texture->Filter = TextureFilter::TF_Bilinear;
		Texture->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;
		Texture->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;
		Texture->UpdateResource();
	}

	void* RawTextureData = Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
	FMemory::Memcpy(RawTextureData, Pixels.GetData(), Pixels.Num() * sizeof(FColor));
	Texture->GetPlatformData()->Mips[0].BulkData.Unlock();
	Texture->UpdateResource();
#endif
}

