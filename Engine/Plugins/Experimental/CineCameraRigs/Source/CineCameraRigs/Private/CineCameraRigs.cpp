// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraRigs.h"
#include "CineSplineLog.h"
#include "ConcertSyncSettings.h"
#include "Algo/AnyOf.h"

#include "CineCameraRigRail.h"
#include "MovieSceneTracksComponentTypes.h"

#define LOCTEXT_NAMESPACE "FCineCameraRigsModule"

DEFINE_LOG_CATEGORY(LogCineSpline);

void FCineCameraRigsModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	// Update MU filter
	UConcertSyncConfig* SyncConfig = GetMutableDefault<UConcertSyncConfig>();
	if(SyncConfig)
	{ 
		const FSoftClassPath MetadataClassPath = FSoftClassPath(TEXT("/Script/CineCameraRigs.CineSplineMetadata"));
		const bool bIncluded = Algo::AnyOf(SyncConfig->IncludeObjectClassFilters, [MetadataClassPath](const FTransactionClassFilter& Filter)
			{
				return Filter.ObjectClasses.Contains(MetadataClassPath);
			});

		if (!bIncluded)
		{
			FTransactionClassFilter Filter;
			Filter.ObjectOuterClass = FSoftClassPath(TEXT("/Script/Engine.World"));
			Filter.ObjectClasses.Add(MetadataClassPath);
			SyncConfig->IncludeObjectClassFilters.Add(Filter);
		}

	}

	using namespace UE::MovieScene;
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
	TracksComponents->Accessors.Float.Add(
		ACineCameraRigRail::StaticClass(), GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, AbsolutePositionOnRail),
		GetAbsolutePositionOnRail, SetAbsolutePositionOnRail);
}

float FCineCameraRigsModule::GetAbsolutePositionOnRail(const UObject* Object)
{
	return CastChecked<const ACineCameraRigRail>(Object)->AbsolutePositionOnRail;
}

void  FCineCameraRigsModule::SetAbsolutePositionOnRail(UObject* Object, float InNewValue)
{
	CastChecked<ACineCameraRigRail>(Object)->SetAbsolutePositionOnRail(InNewValue);
}

void FCineCameraRigsModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCineCameraRigsModule, CineCameraRigs)