// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineSplineComponent.h"

#include "Engine/Engine.h"

UCineSplineComponent::UCineSplineComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ClearSplinePoints(false);
	
	CineSplineMetadata = ObjectInitializer.CreateDefaultSubobject<UCineSplineMetadata>(this, TEXT("CineSplineMetadata"));
	CineSplineMetadata->Reset(10);

	FCineSplinePointData Data;
	FCineSplineCurveDefaults Defaults;
	Data.Location = FVector(0, 0, 0);
	Data.Rotation = FRotator(0, 0, 0);
	Data.FocalLength = Defaults.DefaultFocalLength;
	Data.Aperture = Defaults.DefaultAperture;
	Data.FocusDistance = Defaults.DefaultFocusDistance;

	AddSplineDataAtPosition(1.0f, Data);

	Data.Location.X = 100.0;
	AddSplineDataAtPosition(2.0f, Data);
}


USplineMetadata* UCineSplineComponent::GetSplinePointsMetadata()
{
	return CineSplineMetadata;
}

const USplineMetadata* UCineSplineComponent::GetSplinePointsMetadata() const
{
	return CineSplineMetadata;
}

void UCineSplineComponent::PostLoad()
{
	Super::PostLoad();
	if (CineSplineMetadata)
	{
		SynchronizeProperties();
	}
}

TStructOnScope<FActorComponentInstanceData> UCineSplineComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData = MakeStructOnScope<FActorComponentInstanceData, FCineSplineInstanceData>(this);
	FCineSplineInstanceData* SplineInstanceData = InstanceData.Cast<FCineSplineInstanceData>();

	if (bSplineHasBeenEdited)
	{
		SplineInstanceData->CineSplineMetadata = CineSplineMetadata;
		SplineInstanceData->SplineCurves = GetSplineCurves();
	}

	SplineInstanceData->bSplineHasBeenEdited = bSplineHasBeenEdited;

	return InstanceData;
}

void UCineSplineComponent::ApplyComponentInstanceData(FCineSplineInstanceData* SplineInstanceData, const bool bPostUCS)
{
	check(SplineInstanceData);

	if (bPostUCS)
	{
		if (bInputSplinePointsToConstructionScript)
		{
			// Don't reapply the saved state after the UCS has run if we are inputting the points to it.
			// This allows the UCS to work on the edited points and make its own changes.
			return;
		}
		else
		{
			//bModifiedByConstructionScript = (SplineInstanceData->SplineCurvesPreUCS != SplineCurves);

			// If we are restoring the saved state, unmark the SplineCurves property as 'modified'.
			// We don't want to consider that these changes have been made through the UCS.
			TArray<FProperty*> SplineProperties;
			for (const FName& Property : GetSplinePropertyNames())
			{
				SplineProperties.Add(FindFProperty<FProperty>(StaticClass(), Property));
			}

			RemoveUCSModifiedProperties(SplineProperties);
		}
	}
	else
	{
		//SplineInstanceData->SplineCurvesPreUCS = SplineCurves;
	}

	if (SplineInstanceData->bSplineHasBeenEdited)
	{
		// Copy metadata to current component
		if (CineSplineMetadata && SplineInstanceData->CineSplineMetadata)
		{
			CineSplineMetadata->Modify();
			UEngine::CopyPropertiesForUnrelatedObjects(SplineInstanceData->CineSplineMetadata, CineSplineMetadata);
		}
	
		bModifiedByConstructionScript = false;
	}

	UpdateSpline();
	SynchronizeProperties();
}

void UCineSplineComponent::SynchronizeProperties()
{
	int32 NumOfPoints = GetNumberOfSplinePoints();
	if (CineSplineMetadata && NumOfPoints > 0)
	{
		CineSplineMetadata->Fixup(NumOfPoints, this);

		// Fixing invalid AbsolutePosition
		// This is to make sure CustomPostion value is incrementing from previous point
		float PreviousValue = CineSplineMetadata->AbsolutePosition.Points[0].OutVal;
		for (int32 Index = 1; Index < NumOfPoints; ++Index)
		{
			float CurrentValue = CineSplineMetadata->AbsolutePosition.Points[Index].OutVal;
			if (CurrentValue <= PreviousValue)
			{
				CurrentValue = PreviousValue + 1.0f;
				CineSplineMetadata->AbsolutePosition.Points[Index].OutVal = CurrentValue;
			}
			PreviousValue = CurrentValue;
		}
	}
}

void UCineSplineComponent::SetFocalLengthAtSplinePoint(const int32 PointIndex, const float Value)
{
	int32 NumPoints = CineSplineMetadata-> FocalLength.Points.Num();
	check(PointIndex >= 0 && PointIndex < NumPoints);
	CineSplineMetadata->Modify();
	CineSplineMetadata->FocalLength.Points[PointIndex].OutVal = Value;
}

void UCineSplineComponent::SetApertureAtSplinePoint(const int32 PointIndex, const float Value)
{
	int32 NumPoints = CineSplineMetadata->Aperture.Points.Num();
	check(PointIndex >= 0 && PointIndex < NumPoints);
	CineSplineMetadata->Modify();
	CineSplineMetadata->Aperture.Points[PointIndex].OutVal = Value;
}

void UCineSplineComponent::SetFocusDistanceAtSplinePoint(const int32 PointIndex, const float Value)
{
	int32 NumPoints = CineSplineMetadata->FocusDistance.Points.Num();
	check(PointIndex >= 0 && PointIndex < NumPoints);
	CineSplineMetadata->Modify();
	CineSplineMetadata->FocusDistance.Points[PointIndex].OutVal = Value;
}


void UCineSplineComponent::SetAbsolutePositionAtSplinePoint(const int32 PointIndex, const float Value)
{
	int32 NumPoints = CineSplineMetadata->AbsolutePosition.Points.Num();
	check(PointIndex >= 0 && PointIndex < NumPoints);
	CineSplineMetadata->Modify();
	CineSplineMetadata->AbsolutePosition.Points[PointIndex].OutVal = Value;
}

void UCineSplineComponent::SetPointRotationAtSplinePoint(const int32 PointIndex, const FQuat Value)
{
	int32 NumPoints = CineSplineMetadata->PointRotation.Points.Num();
	check(PointIndex >= 0 && PointIndex < NumPoints);
	CineSplineMetadata->Modify();
	CineSplineMetadata->PointRotation.Points[PointIndex].OutVal = Value;
}

bool UCineSplineComponent::FindSplineDataAtPosition(const float InPosition, int32& OutIndex, const float Tolerance) const
{
	int32 NumPoints = CineSplineMetadata->AbsolutePosition.Points.Num();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		if (FMath::IsNearlyEqual(InPosition, CineSplineMetadata->AbsolutePosition.Points[i].OutVal, Tolerance))
		{
			OutIndex = i;
			return true;
		}
	}
	OutIndex = -1;
	return false;
}

float UCineSplineComponent::GetInputKeyAtPosition(const float InPosition) const
{
	// TODO: handle closed loop

	float OutValue = 0.0f;
	
	int32 NumPoints = CineSplineMetadata->AbsolutePosition.Points.Num();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		if (InPosition < CineSplineMetadata->AbsolutePosition.Points[i].OutVal)
		{
			if (i > 0)
			{
				float Value0 = CineSplineMetadata->AbsolutePosition.Points[i - 1].OutVal;
				float Value1 = CineSplineMetadata->AbsolutePosition.Points[i].OutVal;
				OutValue = (InPosition - Value0) / (Value1 - Value0) + (float)(i-1);
			}
			break;
		}
		OutValue = (float)i;
	}

	return OutValue;
}

float UCineSplineComponent::GetPositionAtInputKey(const float InKey) const
{
	return GetFloatPropertyAtSplineInputKey(InKey, FName(TEXT("AbsolutePosition")));
}

FQuat UCineSplineComponent::GetPointRotationAtSplinePoint(int32 Index) const
{
	const int32 NumPoints = CineSplineMetadata->PointRotation.Points.Num();
	if (NumPoints > 0)
	{
		const int32 ClampedIndex = FMath::Clamp(Index, 0, NumPoints - 1);
		return CineSplineMetadata->PointRotation.Points[ClampedIndex].OutVal;
	}
	
	return FQuat::Identity;
}

FQuat UCineSplineComponent::GetPointRotationAtSplineInputKey(float InKey) const
{
	return CineSplineMetadata->PointRotation.Eval(InKey, FQuat::Identity);
}

void UCineSplineComponent::UpdateSplineDataAtIndex(const int InIndex, const FCineSplinePointData& InPointData)
{
	ESplinePointType::Type PointType = GetSplinePointType(InIndex);
	SetLocationAtSplinePoint(InIndex, InPointData.Location, ESplineCoordinateSpace::World, false);
	SetRotationAtSplinePoint(InIndex, InPointData.Rotation, ESplineCoordinateSpace::World, false);
	SetSplinePointType(InIndex, PointType, false);
	SetFocalLengthAtSplinePoint(InIndex, InPointData.FocalLength);
	SetApertureAtSplinePoint(InIndex, InPointData.Aperture);
	SetFocusDistanceAtSplinePoint(InIndex, InPointData.FocusDistance);
	const FQuat Quat = GetComponentTransform().InverseTransformRotation(InPointData.Rotation.Quaternion());
	SetPointRotationAtSplinePoint(InIndex, Quat);
	UpdateSpline();
}

void UCineSplineComponent::AddSplineDataAtPosition(const float InPosition, const FCineSplinePointData& InPointData)
{
	int32 NewIndex = 0;
	int32 NumPoints = CineSplineMetadata->AbsolutePosition.Points.Num();
	for (int32 i = 0; i < NumPoints; ++i)
	{
		if (InPosition <= CineSplineMetadata->AbsolutePosition.Points[i].OutVal)
		{
			break;
		}
		NewIndex++;
	}
	AddSplinePointAtIndex(InPointData.Location, NewIndex, ESplineCoordinateSpace::World, false);
	SetRotationAtSplinePoint(NewIndex, InPointData.Rotation, ESplineCoordinateSpace::World, false);
	SetFocalLengthAtSplinePoint(NewIndex, InPointData.FocalLength);
	SetApertureAtSplinePoint(NewIndex, InPointData.Aperture);
	SetFocusDistanceAtSplinePoint(NewIndex, InPointData.FocusDistance);
	SetAbsolutePositionAtSplinePoint(NewIndex, InPosition);
	SetSplinePointType(NewIndex, ESplinePointType::Curve, false);
	const FQuat Quat = GetComponentTransform().InverseTransformRotation(InPointData.Rotation.Quaternion());
	SetPointRotationAtSplinePoint(NewIndex, Quat);
	UpdateSpline();
}

FCineSplinePointData UCineSplineComponent::GetSplineDataAtPosition(const float InPosition) const
{
	FCineSplinePointData PointData;
	const float InputKey = GetInputKeyAtPosition(InPosition);
	PointData.Location = GetLocationAtSplineInputKey(InputKey, ESplineCoordinateSpace::World);
	PointData.Rotation = GetPointRotationAtSplineInputKey(InputKey).Rotator();
	PointData.FocalLength = GetFloatPropertyAtSplineInputKey(InputKey, FName(TEXT("FocalLength")));
	PointData.Aperture = GetFloatPropertyAtSplineInputKey(InputKey, FName(TEXT("Aperture")));
	PointData.FocusDistance = GetFloatPropertyAtSplineInputKey(InputKey, FName(TEXT("FocusDistance")));
	return PointData;
}

void UCineSplineComponent::UpdateSpline()
{
	USplineComponent::UpdateSpline();
	if (CineSplineMetadata)
	{
		CineSplineMetadata->AutoSetTangents();
	}
	if (OnSplineEdited.IsBound())
	{
		OnSplineEdited.Broadcast();
	}
	if (OnSplineEdited_BP.IsBound())
	{
		OnSplineEdited_BP.Broadcast();
	}
}


#if WITH_EDITOR
void UCineSplineComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (OnSplineEdited.IsBound())
	{
		OnSplineEdited.Broadcast();
	}
	if (OnSplineEdited_BP.IsBound())
	{
		OnSplineEdited_BP.Broadcast();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
	SynchronizeProperties();

	const FProperty* TouchedProperty = PropertyChangedEvent.Property;
	const FName TouchedName = TouchedProperty ? TouchedProperty->GetFName() : NAME_None;

	static const FName AbsolutePositionName = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, AbsolutePosition);
	static const FName FocalLengthName      = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, FocalLength);
	static const FName ApertureName         = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, Aperture);
	static const FName FocusDistanceName    = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, FocusDistance);
	static const FName PointRotationName    = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, PointRotation);
	static const FName SplineCurvesName     = GET_MEMBER_NAME_CHECKED(USplineComponent, SplineCurves);

	const bool bShouldBroadcast =
		TouchedProperty == nullptr ||
		TouchedName == AbsolutePositionName ||
		TouchedName == FocalLengthName ||
		TouchedName == ApertureName ||
		TouchedName == FocusDistanceName ||
		TouchedName == PointRotationName ||
		TouchedName == SplineCurvesName;

	if (bShouldBroadcast)
	{
		OnSplineUpdated.Broadcast();
	}
}
#endif

void FCineSplineInstanceData::ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase)
{
	if (UCineSplineComponent* SplineComp = CastChecked<UCineSplineComponent>(Component))
	{
		// This ensures there is no stale data causing issues where the spline is marked as read-only even though it shouldn't.
		// There might be a better solution, but this works.
		SplineComp->UpdateSpline();

		Super::ApplyToComponent(Component, CacheApplyPhase);
		SplineComp->ApplyComponentInstanceData(this, (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript));
	}
}


