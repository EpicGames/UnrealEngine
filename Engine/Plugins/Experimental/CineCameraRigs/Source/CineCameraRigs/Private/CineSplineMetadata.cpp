// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineSplineMetadata.h"
#include "CineSplineComponent.h"
#include "CineSplineLog.h"

void UCineSplineMetadata::InsertPoint(int32 Index, float Alpha, bool bClosedLoop)
{
	if (Index < 0)
	{
		UE_LOGF(LogCineSpline, Error, "InsertPoint - invalid Index: %d", Index);
		return;
	}
	Alpha = FMath::Clamp(Alpha, 0.f, 1.f);

	Modify();

	int32 NumPoints = FocalLength.Points.Num();
	float InputKey = static_cast<float>(Index);
	if (Index >= NumPoints)
	{
		// Just add point to the end instead of trying to insert
		AddPoint(InputKey);
	}
	else
	{
		const int32 PrevIndex = (bClosedLoop && Index == 0 ? NumPoints - 1 : Index - 1);
		const bool bHasPrevIndex = (PrevIndex >= 0 && PrevIndex < NumPoints);
		float NewFocalLengthVal = FocalLength.Points[Index].OutVal;
		float NewApertureVal = Aperture.Points[Index].OutVal;
		float NewFocusDistanceVal = FocusDistance.Points[Index].OutVal;
		float NewAbsolutePositionVal = AbsolutePosition.Points[Index].OutVal;
		FQuat NewRotationVal = PointRotation.Points[Index].OutVal;
		if (bHasPrevIndex)
		{
			float PrevFocalLengthVal = FocalLength.Points[PrevIndex].OutVal;
			float PrevApertureVal = Aperture.Points[PrevIndex].OutVal;
			float PrevFocusDistanceVal = FocusDistance.Points[PrevIndex].OutVal;
			float PrevAbsolutePositionVal = AbsolutePosition.Points[PrevIndex].OutVal;
			FQuat PrevRotationVal = PointRotation.Points[PrevIndex].OutVal;
			NewFocalLengthVal = FMath::LerpStable(PrevFocalLengthVal, NewFocalLengthVal, Alpha);
			NewApertureVal = FMath::LerpStable(PrevApertureVal, NewApertureVal, Alpha);
			NewFocusDistanceVal = FMath::LerpStable(PrevFocusDistanceVal, NewFocusDistanceVal, Alpha);
			NewAbsolutePositionVal = FMath::LerpStable(PrevAbsolutePositionVal, NewAbsolutePositionVal, Alpha);
			NewRotationVal = FMath::LerpStable(PrevRotationVal, NewRotationVal, Alpha);
		}
		FInterpCurvePoint<float> NewAbsolutePosition(InputKey, NewAbsolutePositionVal);
		AbsolutePosition.Points.Insert(NewAbsolutePosition, Index);

		FInterpCurvePoint<float> NewFocalLength(InputKey, NewFocalLengthVal);
		FocalLength.Points.Insert(NewFocalLength, Index);

		FInterpCurvePoint<float> NewAperture(InputKey, NewApertureVal);
		Aperture.Points.Insert(NewAperture, Index);

		FInterpCurvePoint<float> NewFocusDistance(InputKey, NewFocusDistanceVal);
		FocusDistance.Points.Insert(NewFocusDistance, Index);

		FInterpCurvePoint<FQuat> NewRotation(InputKey, NewRotationVal);
		PointRotation.Points.Insert(NewRotation, Index);


		for (int32 i = Index + 1; i < FocalLength.Points.Num(); ++i)
		{
			FocalLength.Points[i].InVal += 1.0f;
			Aperture.Points[i].InVal += 1.0f;
			FocusDistance.Points[i].InVal += 1.0f;
			AbsolutePosition.Points[i].InVal += 1.0f;
			PointRotation.Points[i].InVal += 1.0f;
		}
		AutoSetTangents();
	}
}

void UCineSplineMetadata::UpdatePoint(int32 Index, float Alpha, bool bClosedLoop)
{
	int32 NumPoints = FocalLength.Points.Num();
	if (!FocalLength.Points.IsValidIndex(Index))
	{
		UE_LOGF(LogCineSpline, Error, "UpdatePoint - invalid Index: %d", Index);
		return;
	}
	Alpha = FMath::Clamp(Alpha, 0.f, 1.f);

	int32 PrevIndex = (bClosedLoop && Index == 0 ? NumPoints - 1 : Index - 1);
	int32 NextIndex = (bClosedLoop && Index + 1 > NumPoints ? 0 : Index + 1);

	bool bHasPrevIndex = (PrevIndex >= 0 && PrevIndex < NumPoints);
	bool bHasNextIndex = (NextIndex >= 0 && NextIndex < NumPoints);

	Modify();

	if (bHasPrevIndex && bHasNextIndex)
	{
		float PrevAbsolutePositionVal = AbsolutePosition.Points[PrevIndex].OutVal;
		float PrevFocalLengthVal = FocalLength.Points[PrevIndex].OutVal;
		float PrevApertureVal = Aperture.Points[PrevIndex].OutVal;
		float PrevFocusDistanceVal = FocusDistance.Points[PrevIndex].OutVal;
		FQuat PrevRotationVal = PointRotation.Points[PrevIndex].OutVal;
		float NextAbsolutePositionVal = AbsolutePosition.Points[NextIndex].OutVal;
		float NextFocalLengthVal = FocalLength.Points[NextIndex].OutVal;
		float NextApertureVal = Aperture.Points[NextIndex].OutVal;
		float NextFocusDistanceVal = FocusDistance.Points[NextIndex].OutVal;
		FQuat NextRotationVal = PointRotation.Points[NextIndex].OutVal;

		AbsolutePosition.Points[Index].OutVal = FMath::LerpStable(PrevAbsolutePositionVal, NextAbsolutePositionVal, Alpha);
		FocalLength.Points[Index].OutVal = FMath::LerpStable(PrevFocalLengthVal, NextFocalLengthVal, Alpha);
		Aperture.Points[Index].OutVal = FMath::LerpStable(PrevApertureVal, NextApertureVal, Alpha);
		FocusDistance.Points[Index].OutVal = FMath::LerpStable(PrevFocusDistanceVal, NextFocusDistanceVal, Alpha);
		PointRotation.Points[Index].OutVal = FMath::LerpStable(PrevRotationVal, NextRotationVal, Alpha);
	}
	AutoSetTangents();
}

void UCineSplineMetadata::AddPoint(float InputKey)
{
	Modify();

	float NewAbsolutePositionVal = -1.0f;
	float NewFocalLengthVal = 35.0f;
	float NewApertureVal = 2.8f;
	float NewFocusDistanceVal = 100000.f;
	FQuat NewRotationVal = FQuat::Identity;
	
	int Index = FocalLength.Points.Num() - 1;
	if (Index >= 0)
	{
		NewFocalLengthVal = FocalLength.Points[Index].OutVal;
		NewApertureVal = Aperture.Points[Index].OutVal;
		NewFocusDistanceVal = FocusDistance.Points[Index].OutVal;
		NewAbsolutePositionVal = AbsolutePosition.Points[Index].OutVal + 1.0f;
		NewRotationVal = PointRotation.Points[Index].OutVal;
	}

	float NewInputKey = static_cast<float>(++Index);
	FocalLength.Points.Emplace(NewInputKey, NewFocalLengthVal);
	Aperture.Points.Emplace(NewInputKey, NewApertureVal);
	FocusDistance.Points.Emplace(NewInputKey, NewFocusDistanceVal);
	AbsolutePosition.Points.Emplace(NewInputKey, NewAbsolutePositionVal);
	PointRotation.Points.Emplace(NewInputKey, NewRotationVal);

	AutoSetTangents();
}

void UCineSplineMetadata::RemovePoint(int32 Index)
{
	check(Index < FocalLength.Points.Num());

	Modify();
	AbsolutePosition.Points.RemoveAt(Index);
	FocalLength.Points.RemoveAt(Index);
	Aperture.Points.RemoveAt(Index);
	FocusDistance.Points.RemoveAt(Index);
	PointRotation.Points.RemoveAt(Index);

	for (int32 i = Index; i < FocalLength.Points.Num(); ++i)
	{
		AbsolutePosition.Points[i].InVal -= 1.0f;
		FocalLength.Points[i].InVal -= 1.0f;
		Aperture.Points[i].InVal -= 1.0f;
		FocusDistance.Points[i].InVal -= 1.0f;
		PointRotation.Points[i].InVal -= 1.0f;
	}

	AutoSetTangents();
}

void UCineSplineMetadata::DuplicatePoint(int32 Index)
{
	check(Index < FocalLength.Points.Num());

	int32 NumPoints = FocalLength.Points.Num();
	float NewValue = AbsolutePosition.Points[Index].OutVal + 1.0f;
	if (Index < NumPoints - 1)
	{
		NewValue = (AbsolutePosition.Points[Index].OutVal + AbsolutePosition.Points[Index + 1].OutVal) * 0.5;
	}

	Modify();
	AbsolutePosition.Points.Insert(FInterpCurvePoint<float>(AbsolutePosition.Points[Index]), Index);
	FocalLength.Points.Insert(FInterpCurvePoint<float>(FocalLength.Points[Index]), Index);
	Aperture.Points.Insert(FInterpCurvePoint<float>(Aperture.Points[Index]), Index);
	FocusDistance.Points.Insert(FInterpCurvePoint<float>(FocusDistance.Points[Index]), Index);
	PointRotation.Points.Insert(FInterpCurvePoint<FQuat>(PointRotation.Points[Index]), Index);
	AbsolutePosition.Points[Index+1].OutVal = NewValue;


	for (int32 i = Index + 1; i < FocalLength.Points.Num(); ++i)
	{
		AbsolutePosition.Points[i].InVal += 1.0f;
		FocalLength.Points[i].InVal += 1.0f;
		Aperture.Points[i].InVal += 1.0f;
		FocusDistance.Points[i].InVal += 1.0f;
		PointRotation.Points[i].InVal += 1.0f;
	}
	AutoSetTangents();
}

void UCineSplineMetadata::CopyPoint(const USplineMetadata* FromSplineMetadata, int32 FromIndex, int32 ToIndex)
{
	check(FromSplineMetadata != nullptr);

	if (const UCineSplineMetadata* FromMetadata = Cast<UCineSplineMetadata>(FromSplineMetadata))
	{
		check(ToIndex < FocalLength.Points.Num());
		check(FromIndex < FromMetadata->FocalLength.Points.Num());

		Modify();
		FocalLength.Points[ToIndex].OutVal = FromMetadata->FocalLength.Points[FromIndex].OutVal;
		Aperture.Points[ToIndex].OutVal = FromMetadata->Aperture.Points[FromIndex].OutVal;
		FocusDistance.Points[ToIndex].OutVal = FromMetadata->FocusDistance.Points[FromIndex].OutVal;
		AbsolutePosition.Points[ToIndex].OutVal = FromMetadata->AbsolutePosition.Points[FromIndex].OutVal;
		PointRotation.Points[ToIndex].OutVal = FromMetadata->PointRotation.Points[FromIndex].OutVal;

		AutoSetTangents();
	}
}

void UCineSplineMetadata::Reset(int32 NumPoints)
{
	Modify();
	FocalLength.Points.Reset(NumPoints);
	Aperture.Points.Reset(NumPoints);
	FocusDistance.Points.Reset(NumPoints);
	AbsolutePosition.Points.Reset(NumPoints);
	PointRotation.Points.Reset(NumPoints);
}

void UCineSplineMetadata::AutoSetTangents()
{
	Modify();
	FocalLength.AutoSetTangents(0.0f, false);
	Aperture.AutoSetTangents(0.0f, false);
	FocusDistance.AutoSetTangents(0.0f, false);
	PointRotation.AutoSetTangents(0.0f, false);
}

void UCineSplineMetadata::SetTangentType(int32 Index, ECineSplineMetadataTangentType TangentType)
{
	check(Index < FocalLength.Points.Num());
	check(Index >= 0);

	EInterpCurveMode Mode = ConvertCineSplineMetadataTangentTypeToInterpCurveMode(TangentType);
	if (Mode != CIM_Unknown)
	{
		Modify();
		FocalLength.Points[Index].InterpMode = Mode;
		Aperture.Points[Index].InterpMode = Mode;
		FocusDistance.Points[Index].InterpMode = Mode;
		PointRotation.Points[Index].InterpMode = Mode;
	}
}

EInterpCurveMode ConvertCineSplineMetadataTangentTypeToInterpCurveMode(ECineSplineMetadataTangentType MetadataTangentType)
{
	switch (MetadataTangentType)
	{
		case ECineSplineMetadataTangentType::Linear:
			return CIM_Linear;
		case ECineSplineMetadataTangentType::Curve:
			return CIM_CurveAuto;
		case ECineSplineMetadataTangentType::Constant:
			return CIM_Constant;
		case ECineSplineMetadataTangentType::CurveClamped:
			return CIM_CurveAutoClamped;

		default:
			return CIM_Unknown;
	}
}

ECineSplineMetadataTangentType ConvertInterpCurveModeToCineSplineMetadataTangentType(EInterpCurveMode InterpCurveMode)
{
	switch (InterpCurveMode)
	{
		case CIM_Linear:
			return ECineSplineMetadataTangentType::Linear;
		case CIM_CurveAuto:
			return ECineSplineMetadataTangentType::Curve;
		case CIM_Constant:
			return ECineSplineMetadataTangentType::Constant;
		case CIM_CurveAutoClamped:
			return ECineSplineMetadataTangentType::CurveClamped;

		default:
			return ECineSplineMetadataTangentType::Linear;
	}
}

#if WITH_EDITORONLY_DATA
template <class T>
void FixupCurve(FInterpCurve<T>& Curve, const T& DefaultValue, int32 NumPoints)
{
	// Fixup bad InVal values from when the add operation below used the wrong value
	for (int32 PointIndex = 0; PointIndex < Curve.Points.Num(); PointIndex++)
	{
		float InVal = PointIndex;
		Curve.Points[PointIndex].InVal = InVal;
	}

	while (Curve.Points.Num() < NumPoints)
	{
		// InVal is the point index which is ascending so use previous point plus one.
		float InVal = Curve.Points.Num() > 0 ? Curve.Points[Curve.Points.Num() - 1].InVal + 1.0f : 0.0f;
		Curve.Points.Add(FInterpCurvePoint<T>(InVal, DefaultValue));
	}

	if (Curve.Points.Num() > NumPoints)
	{
		Curve.Points.RemoveAt(NumPoints, Curve.Points.Num() - NumPoints);
	}
}
#endif

void UCineSplineMetadata::Fixup(int32 NumPoints, USplineComponent* SplineComp)
{
	const FCineSplineCurveDefaults& Defaults = CastChecked<UCineSplineComponent>(SplineComp)->CameraSplineDefaults;
#if WITH_EDITORONLY_DATA
	FixupCurve(FocalLength, Defaults.DefaultFocalLength, NumPoints);
	FixupCurve(Aperture, Defaults.DefaultAperture, NumPoints);
	FixupCurve(FocusDistance, Defaults.DefaultFocusDistance, NumPoints);
	FixupCurve(AbsolutePosition, Defaults.DefaultAbsolutePosition, NumPoints);
	FixupCurve(PointRotation, Defaults.DefaultPointRotation, NumPoints);
#endif
}

#if WITH_EDITOR
void UCineSplineMetadata::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FProperty* TouchedProperty = PropertyChangedEvent.Property;
	const FName TouchedName = TouchedProperty ? TouchedProperty->GetFName() : NAME_None;

	static const FName AbsolutePositionName = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, AbsolutePosition);
	static const FName FocalLengthName      = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, FocalLength);
	static const FName ApertureName         = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, Aperture);
	static const FName FocusDistanceName    = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, FocusDistance);
	static const FName PointRotationName    = GET_MEMBER_NAME_CHECKED(UCineSplineMetadata, PointRotation);
	static const FName SplineCurvesName     = GET_MEMBER_NAME_CHECKED(USplineComponent, SplineCurves);

	const bool bNeedsFixup =
		TouchedProperty == nullptr ||
		TouchedName == AbsolutePositionName ||
		TouchedName == FocalLengthName ||
		TouchedName == ApertureName ||
		TouchedName == FocusDistanceName ||
		TouchedName == PointRotationName ||
		TouchedName == SplineCurvesName;

	if (bNeedsFixup)
	{
		if (UCineSplineComponent* OwningSpline = GetTypedOuter<UCineSplineComponent>())
		{
			const int32 NumOfPoints = OwningSpline->GetNumberOfSplinePoints();
			if (NumOfPoints > 0)
			{
				Fixup(NumOfPoints, OwningSpline);
			}
		}
	}
}
#endif