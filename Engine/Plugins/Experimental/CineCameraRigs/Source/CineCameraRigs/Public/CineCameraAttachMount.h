// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Constraint.h"

#include "CineCameraAttachMount.generated.h"

#define UE_API CINECAMERARIGS_API

class USpringArmComponent;
class UBillboardComponent;
class UTickableParentConstraint;

UCLASS(Blueprintable, Category = "VirtualProduction")
class ACineCameraAttachMount : public AActor
{
	GENERATED_BODY()

public:
	UE_API ACineCameraAttachMount(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool ShouldTickIfViewportsOnly() const override;
	UE_API virtual class USceneComponent* GetDefaultAttachComponent() const override;

	/** Constraint target actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Constraint")
	TSoftObjectPtr<AActor> TargetActor;

	/** Constraint target socket name*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Constraint")
	FName TargetSocket;

	/** Constraint axis filter */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetTransformFilter, Category = "Constraint")
	FTransformFilter TransformFilter;

	/** SprintArm component for lag effect */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Default")
	TObjectPtr<USpringArmComponent> SpringArmComponent;

	/** If true, attached actors lag behind target position to smooth its movement */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetEnableLocationLag, Category = "Lag")
	bool bEnableLocationLag = true;

	/** If true, attached actors lag behind target position to smooth its movement */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetEnableRotationLag, Category = "Lag")
	bool bEnableRotationLag = true;
	
	/** If bEnableLocationLag is true, controls how quickly camera reaches target position. Low values are slower (more lag), high values are faster (less lag), while zero is instant (no lag) */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetLocationLagSpeed, Category = "Lag", meta = (editcondition = "bEnableLocationLag", ClampMin = "0.0", ClampMax = "1000.0", UIMin = "0.0", UIMax = "1000.0"))
	float LocationLagSpeed = 10.0f;

	/** If bEnableRotationLag is true, controls how quickly camera reaches target position. Low values are slower (more lag), high values are faster (less lag), while zero is instant (no lag)*/
	UPROPERTY(EditAnywhere, BlueprintSetter=SetRotationLagSpeed, Category = "Lag", meta = (editcondition = "bEnableRotationLag", ClampMin = "0.0", ClampMax = "1000.0", UIMin = "0.0", UIMax = "1000.0"))
	float RotationLagSpeed = 10.0f;

	/** Set bEnableLocationLag */
	UFUNCTION(BlueprintSetter)
	UE_API void SetEnableLocationLag(bool bEnabled);

	/** Set bEnableRotationLag */
	UFUNCTION(BlueprintSetter)
	UE_API void SetEnableRotationLag(bool bEnabled);

	/** Set LocationLagSpeed */
	UFUNCTION(BlueprintSetter)
	UE_API void SetLocationLagSpeed(float Speed);

	/** Set RotationLagSpeed*/
	UFUNCTION(BlueprintSetter)
	UE_API void SetRotationLagSpeed(float Speed);

	/** Set TransformFilter for the internal constraint*/
	UFUNCTION(BlueprintSetter)
	UE_API void SetTransformFilter(const FTransformFilter& InFilter);


#if WITH_EDITORONLY_DATA
	/** If enabled, it shows the preview meshes */
	UPROPERTY(Transient, EditAnywhere, Category = "Preview")
	bool bShowPreviewMeshes;

	/** Scale of the preview meshes */
	UPROPERTY(Transient, EditAnywhere, Category = "Preview")
	float PreviewMeshScale;

	/** Preview mesh for this actor itself*/
	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewMesh_Root;

	/** Preview mesh*/
	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> MountMesh;

	/** Preview mount mesh components*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<UStaticMeshComponent>> PreviewMeshes_Mount;

#endif 

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	/** Get ParentConstraint object. Returns nullptr if there is no constraint*/
	UFUNCTION(BlueprintPure, Category = "Constraint")
	UE_API UTickableParentConstraint* GetConstraint();

	/** Reset constraint offset for location */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Constraint")
	UE_API void ResetLocationOffset();

	/** Reset constraint offset for rotation */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Constraint")
	UE_API void ResetRotationOffset();

	/** Zero out rotationX in world space. This can help setting the mount parallel to the ground. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Constraint")
	UE_API void ZeroRoll();

	/** Zero out rotationY in world space. This can help setting the mount parallel to the ground. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Constraint")
	UE_API void ZeroTilt();

private:
	bool bConstraintUpdated = false;

	void CreateConstraint();
	void UpdateAxisFilter();
	
#if WITH_EDITOR
	void CreatePreviewMesh();
	void UpdatePreviewMeshes();
#endif

};

#undef UE_API