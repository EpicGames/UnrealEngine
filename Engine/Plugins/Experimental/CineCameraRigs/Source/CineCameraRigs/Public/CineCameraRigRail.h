// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/TimerHandle.h"
#include "UObject/ObjectMacros.h"

#include "CameraRig_Rail.h"
#include "CineSplineComponent.h"

#include "CineCameraRigRail.generated.h"

#define UE_API CINECAMERARIGS_API

class UMovieSceneSequence;
class UMovieSceneFloatTrack;

UENUM(BlueprintType)
enum class ECineCameraRigRailDriveMode : uint8
{
	/** Manual Mode*/
	Manual,

	/** Duration Mode. AbsolutePostionOnRail is updated based on the spline duration */
	Duration,

	/** Speed Mode. AbsolutePositionOnRail is updated based on the specified speed*/
	Speed,
};

UENUM(BlueprintType)
enum class ECineCameraRigRailLoopMode : uint8
{
	/** Play once*/
	NoLoop,

	/** Play Loop*/
	Loop,

	/** Play forward then backward*/
	Bounce,
};

UCLASS(Blueprintable, Category = "VirtualProduction")
class UE_API ACineCameraRigRail : public ACameraRig_Rail
{
	GENERATED_BODY()

public:
	ACineCameraRigRail(const FObjectInitializer& ObjectInitializer);

	virtual void Tick(float DeltaTime) override;
	virtual void PostLoad() override;

	/* Returns CineSplineComponent*/
	UFUNCTION(BlueprintPure, Category = "Rail Components")
	UCineSplineComponent* GetCineSplineComponent() const { return CineSplineComponent; }
	
	UFUNCTION(BlueprintSetter, Category = "Rail Components")
	void SetSplineComponent(UCineSplineComponent* NewSplineComponent) { CineSplineComponent = NewSplineComponent; }

	/* Use AbsolutePosition metadata to parameterize the spline*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rail Controls")
	bool bUseAbsolutePosition = true;

	/* Custom parameter to drive current position*/
	UPROPERTY(EditAnywhere, Interp, BlueprintSetter="SetAbsolutePositionOnRail", Category = "Rail Controls", meta = (EditCondition = "bUseAbsolutePosition"))
	float AbsolutePositionOnRail = 1.0f;

	/* Use PointRotation metadata for attachment orientation. If false, attachment orientation is based on the spline curvature*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rail Controls")
	bool bUsePointRotation = true;

	/* Material assigned to spline component mesh*/
	UPROPERTY(BlueprintSetter=SetSplineMeshMaterial, Category = "SplineVisualization")
	TObjectPtr<UMaterialInterface> SplineMeshMaterial;

	/* Material Instance Dynamic created for the spline mesh */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "SplineVisualization")
	TArray<TObjectPtr<UMaterialInstanceDynamic>> SplineMeshMIDs;

	/* Texture that can be set to SplineMeshMIDs */
	UPROPERTY(BlueprintSetter=SetSplineMeshTexture, Category = "SplineVisualization")
	TObjectPtr<UTexture2D> SplineMeshTexture;

	/* Enable speed visualization. Automatically disabled when position property is driven in Sequencer*/
	UPROPERTY(EditAnywhere, BlueprintSetter=SetDisplaySpeedHeatmap, Category = "SplineVisualization", meta=(EditCondition="!IsSequencerDriven()"))
	bool bDisplaySpeedHeatmap = true;

	/* Number of speed samples per spline segment*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SplineVisualization", meta=(ClampMin=1, EditCondition="bDisplaySpeedHeatmap"))
	int32 SpeedSampleCountPerSegment = 4;

	/* Determines if camera mount inherits LocationX*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Location")
	bool bAttachLocationX = true;

	/* Determines if camera mount inherits LocationY*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Location")
	bool bAttachLocationY = true;

	/* Determines if camera mount inherits LocationZ*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Location")
	bool bAttachLocationZ = true;

	/* Determines if camera mount inherits RotationX*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Rotation")
	bool bAttachRotationX = true;

	/* Determines if camera mount inherits RotationY*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Rotation")
	bool bAttachRotationY = true;

	/* Determines if camera mount inherits RotationZ*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Rotation")
	bool bAttachRotationZ = true;

	/* Determines if it can drive focal length on the attached actors*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Camera")
	bool bInheritFocalLength = true;

	/* Determines if it can drive aperture on the attached actors*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Camera")
	bool bInheritAperture = true;

	/* Determines if it can drive focus distance on the attached actors*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Attachment|Camera")
	bool bInheritFocusDistance = true;

	/* Drive Mode to update position in tick*/
	UPROPERTY(EditAnywhere, BlueprintSetter=SetDriveMode, Category = "DriveMode")
	ECineCameraRigRailDriveMode DriveMode = ECineCameraRigRailDriveMode::Manual;

	/* Specifies the drive speed of the rig rail in centimeter per second */
	UPROPERTY(EditAnywhere, Interp, Category = "DriveMode")
	float Speed = 100;

	/* Determine if it can update position in Duration mode or Speed mode. If false, it pauses the update.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DriveMode")
	bool bPlay = true;

	/* Determine if it plays in reverse.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DriveMode", meta = (EditCondition="LoopMode != ECineCameraRigRailLoopMode::Bounce"))
	bool bReverse = false;

	/* Enable loop in speed or duration mode */
	UPROPERTY(BlueprintReadWrite, Category = "DriveMode", meta = (DeprecatedProperty, DeprecationMessage = "Use LoopMode instead"))
	bool bLoop = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DriveMode")
	ECineCameraRigRailLoopMode LoopMode = ECineCameraRigRailLoopMode::Loop;

	/* If enabled, it compensates world time dilation in Speed/Duration mode so that the spline moves as intended speed regardless of recording time scale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DriveMode")
	bool bCompensateTimeScale = false;

	/* Set spline mesh material*/
	UFUNCTION(BlueprintSetter)
	void SetSplineMeshMaterial(UMaterialInterface* InMaterial);

	/* Set texture used in the spline mesh material */
	UFUNCTION(BlueprintSetter)
	void SetSplineMeshTexture(UTexture2D* InTexture);

	/* Calculate internal velocity at the given position */
	UFUNCTION(BlueprintCallable, Category = "CineCameraRigRail")
	FVector GetVelocityAtPosition(const float InPosition, const float delta = 0.001) const;

	/* Set drive mode*/
	UFUNCTION(BlueprintSetter)
	void SetDriveMode(ECineCameraRigRailDriveMode InMode);

	/* Enable display speed heatmap*/
	UFUNCTION(BlueprintSetter)
	void SetDisplaySpeedHeatmap(bool bEnable);

	/* Returns true if the rig rail is driven by Sequencer */
	UFUNCTION()
	bool IsSequencerDriven();

	/* Set AbsolutePositionOnRail value*/
	UFUNCTION(BlueprintSetter)
	void SetAbsolutePositionOnRail(float Value);

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	UPROPERTY(Category = "CineSpline", VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<class UCineSplineComponent> CineSplineComponent;

	virtual void UpdateRailComponents() override;

	void UpdateSplineMeshMID();
	void SetMIDParameters();
	void UpdateSpeedHeatmap();

	void OnSplineEdited();

	void DriveByParam(float DeltaTime);
	void DriveBySpeed(float DeltaTime);
	float StartPositionValue() const;
	float LastPositionValue() const;
	float SpeedProgress = 0.0f;
	void UpdateSpeedProgress();
	float AdjustDeltaTime(float InDeltaTime) const;

	bool bBounceForward = true;
	bool PlayForward() const;

#if WITH_EDITOR
	void ConditionalRegisterTimerManager();
	UMovieSceneFloatTrack* FindPositionTrack(const UMovieSceneSequence* InSequence);

	FTimerHandle SequencerCheckHandle;

	/* Check if the rig rail is driven by Sequencer*/
	UFUNCTION()
	void OnSequencerCheck();

	bool bSequencerDriven = false;
#endif

};

#undef UE_API