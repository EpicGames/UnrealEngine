// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/Texture2D.h"

#include "CineCameraRigRailHelpers.generated.h"

#define UE_API CINECAMERARIGS_API

UCLASS()
class UCineCameraRigRailHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/* Create a transient heatmap texture from data values*/
	UFUNCTION(BlueprintCallable, Category = "CineCameraRigRail")
	static UE_API void CreateOrUpdateSplineHeatmapTexture(UPARAM(ref) UTexture2D*& Texture, const TArray<float>& DataValues, const float LowValue, const float AverageValue, const float HighValue);
};

#undef UE_API