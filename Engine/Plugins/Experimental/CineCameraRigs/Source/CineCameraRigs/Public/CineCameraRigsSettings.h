// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

#include "CineCameraRigsSettings.generated.h"

class UMaterialInterface;
class UTexture2D;

UCLASS(BlueprintType, Config = CineCameraRigs)
class UCineCameraRigRailSettings : public UObject
{
	GENERATED_BODY()
public:
	/* Path to the default spline mesh material used in CineCameraRigRail*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = CineCameraRigRail)
	TSoftObjectPtr<UMaterialInterface> DefaultSplineMeshMaterial;

	/* Path to the default spline mesh texture used when speed visualization is off*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = CineCameraRigRail)
	TSoftObjectPtr<UTexture2D> DefaultSplineMeshTexture;

	/* Path to the texture used in the speed visualization when drive mode is Speed*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = CineCameraRigRail)
	TSoftObjectPtr<UTexture2D> SpeedModeSplineMeshTexture;
};