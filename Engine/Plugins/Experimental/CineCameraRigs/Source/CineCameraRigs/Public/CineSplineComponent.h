// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/CoreDelegates.h"

#include "Components/SplineComponent.h"

#include "CineSplineMetadata.h"
#include "CineSplinePointData.h"

#include "CineSplineComponent.generated.h"

#define UE_API CINECAMERARIGS_API

UCLASS(ClassGroup = Utility, ShowCategories = (Mobility), HideCategories = (Physics, Collision, Lighting, Rendering, Mobile), meta = (BlueprintSpawnableComponent))
class UCineSplineComponent : public USplineComponent
{
	GENERATED_BODY()

public:
	UE_API UCineSplineComponent(const FObjectInitializer& ObjectInitializer);

	/**
	* Defaults which are used to propagate values to spline points on instances of this in the world
	*/
	UPROPERTY(Category = Camera, EditDefaultsOnly)
	FCineSplineCurveDefaults CameraSplineDefaults;

	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual void PostLoad() override;
	// End of UObject interface

	/** Spline component interface */
	UE_API virtual USplineMetadata* GetSplinePointsMetadata() override;
	UE_API virtual const USplineMetadata* GetSplinePointsMetadata() const override;
	virtual bool AllowsSplinePointScaleEditing() const override { return false; }

	UE_API void ApplyComponentInstanceData(struct FCineSplineInstanceData* ComponentInstanceData, const bool bPostUCS);

	// UActorComponent Interface
	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;

	/** Pointer to metadata */
	UPROPERTY(Instanced)
	TObjectPtr<UCineSplineMetadata> CineSplineMetadata;

	/** Set focal lenght metadata at a given splint point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void SetFocalLengthAtSplinePoint(const int32 PointIndex, const float Value );

	/** Set aperture metadata at a given spline point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void SetApertureAtSplinePoint(const int32 PointIndex, const float value );

	/** Set focus distance metadata at a given spline point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void SetFocusDistanceAtSplinePoint(const int32 PointIndex, const float value);

	/** Set custom position metadata at a given spline point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void SetAbsolutePositionAtSplinePoint(const int32 PointIndex, const float value);

	/* Set camera rotation metadata at a given spline point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void SetPointRotationAtSplinePoint(const int32 PointIndex, const FQuat value);

	/** Returns true if there is a spline point at given custom position*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "CineSpline")
	UE_API bool FindSplineDataAtPosition(const float InPosition, int32& OutIndex, const float Tolerance=0.001f) const;

	/* Update spline point data at the given spline point*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void UpdateSplineDataAtIndex(const int InIndex, const FCineSplinePointData& InPointData );

	/* Add a new spline point data at the given custom position*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API void AddSplineDataAtPosition(const float InPosition, const FCineSplinePointData& InPointData);

	/* Get a spline point data at the given custom position*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API FCineSplinePointData GetSplineDataAtPosition(const float InPosition) const;

	/* Get input key value from custom position */
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API float GetInputKeyAtPosition(const float InPosition) const;

	/* Get custom position value at spline input key */
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API float GetPositionAtInputKey(const float InKey) const;

	/* Get camera rotation metadata property value along the spline at spline point */
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API FQuat GetPointRotationAtSplinePoint(int32 Index) const;

	/* Get camera rotation metadata value along the spline at spline input key*/
	UFUNCTION(BlueprintCallable, Category = "CineSpline")
	UE_API FQuat GetPointRotationAtSplineInputKey(float InKey) const;


	DECLARE_MULTICAST_DELEGATE(FOnSplineEdited);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSplineEdited_BP);

	FOnSplineEdited OnSplineEdited;

	/* Event trigerred when spline is edited */
	UPROPERTY(BlueprintAssignable, Category = "CineSpline", meta = (DisplayName = "OnSplineEdited"))
	FOnSplineEdited_BP OnSplineEdited_BP;

	/** Update the spline tangents and SplineReparamTable. And trigers OnSplineEdited delegates */
	UE_API virtual void UpdateSpline() override;

#if WITH_EDITORONLY_DATA
	/* Toggle if visualizer displays normalized position or absolute position */
	UPROPERTY(EditAnywhere, Category = "CineSplineEditor", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizeNormalizedPosition;

	/* Whether spline length value should be displayed in visualizer */
	UPROPERTY(EditAnywhere, Category = "CineSplineEditor", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizeSplineLength;

	/* Whether point rotation arrows should be displayed in visualizer */
	UPROPERTY(EditAnywhere, Category = "CineSplineEditor", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizePointRotation;

#endif

protected:
	void SynchronizeProperties();

};

USTRUCT()
struct FCineSplineInstanceData : public FSplineInstanceData
{
	GENERATED_BODY()
public:
	FCineSplineInstanceData() = default;
	explicit FCineSplineInstanceData(const UCineSplineComponent* SourceComponent) : FSplineInstanceData(SourceComponent) {}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override;

	UPROPERTY()
	TObjectPtr<UCineSplineMetadata> CineSplineMetadata;
};

#undef UE_API