// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "Components/SplineComponent.h"

#include "CineSplineMetadata.generated.h"

#define UE_API CINECAMERARIGS_API

USTRUCT()
struct FCineSplineCurveDefaults
{
	GENERATED_BODY()

		FCineSplineCurveDefaults()
		: DefaultAbsolutePosition(-1.0f)
		, DefaultFocalLength(35.0f)
		, DefaultAperture(2.8f)
		, DefaultFocusDistance(100000.f)
		, DefaultPointRotation(FQuat::Identity)
	{};

	UPROPERTY(EditDefaultsOnly, Category = CineSplineCurveDefaults)
	float DefaultAbsolutePosition;

	UPROPERTY(EditDefaultsOnly, Category = CineSplineCurveDefaults)
	float DefaultFocalLength;

	UPROPERTY(EditDefaultsOnly, Category = CineSplineCurveDefaults)
	float DefaultAperture;

	UPROPERTY(EditDefaultsOnly, Category = CineSplineCurveDefaults)
	float DefaultFocusDistance;

	UPROPERTY(EditDefaultsOnly, Category = CineSplineCurveDefaults)
	FQuat DefaultPointRotation;
};

/** Interpolation type for CineSpline Metadata */
UENUM()
enum class ECineSplineMetadataTangentType : uint8
{
	Linear,
	Curve,
	Constant,
	CurveClamped
};

/** Convert ECineSplineMetadataTangentType value to EInterpCurveMode value */
UE_API EInterpCurveMode ConvertCineSplineMetadataTangentTypeToInterpCurveMode(ECineSplineMetadataTangentType MetadataTangentType);

/** Convert EInterpCurveMode value to ECineSplineMetadataTangentType value */
UE_API ECineSplineMetadataTangentType ConvertInterpCurveModeToCineSplineMetadataTangentType(EInterpCurveMode InterpCurveMode);

UCLASS()
class UCineSplineMetadata : public USplineMetadata
{
	GENERATED_BODY()
	
public:
	/** Insert point before index, lerping metadata between previous and next key values */
	UE_API virtual void InsertPoint(int32 Index, float Alpha, bool bClosedLoop) override;
	/** Update point at index by lerping metadata between previous and next key values */
	UE_API virtual void UpdatePoint(int32 Index, float Alpha, bool bClosedLoop) override;
	UE_API virtual void AddPoint(float InputKey) override;
	UE_API virtual void RemovePoint(int32 Index) override;
	UE_API virtual void DuplicatePoint(int32 Index) override;
	UE_API virtual void CopyPoint(const USplineMetadata* FromSplineMetadata, int32 FromIndex, int32 ToIndex) override;
	UE_API virtual void Reset(int32 NumPoints) override;
	UE_API virtual void Fixup(int32 NumPoints, USplineComponent* SplineComp) override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY(EditAnywhere, Category = "Point")
	FInterpCurveFloat AbsolutePosition;

	UPROPERTY(EditAnywhere, Category = "Camera")
	FInterpCurveFloat FocalLength;

	UPROPERTY(EditAnywhere, Category = "Camera")
	FInterpCurveFloat Aperture;

	UPROPERTY(EditAnywhere, Category = "Camera")
	FInterpCurveFloat FocusDistance;

	UPROPERTY(EditAnywhere, Category = "Camera")
	FInterpCurveQuat PointRotation;

	/** Automatically set the tangents on the metadata parameters based on surrounding points */
	UE_API void AutoSetTangents();

	/** Specify tangent type on the spline point metadata */
	UE_API void SetTangentType(int32 Index, ECineSplineMetadataTangentType TangentType);
};

#undef UE_API