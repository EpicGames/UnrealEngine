// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "CineSplinePointData.generated.h"

/* Simple struct to hold spline point data*/
USTRUCT(BlueprintType)
struct FCineSplinePointData
{
	GENERATED_BODY()

	FCineSplinePointData()
		: Location(FVector::ZeroVector)
		, Rotation(FRotator::ZeroRotator)
		,FocalLength(35.0f)
		, Aperture(2.8f)
		, FocusDistance(100000.f)
	{};

	UPROPERTY(BlueprintReadWrite, Category="CineSpline")
	FVector Location;

	UPROPERTY(BlueprintReadWrite, Category = "CineSpline")
	FRotator Rotation;

	UPROPERTY(BlueprintReadWrite, Category = "CineSpline")
	float FocalLength;

	UPROPERTY(BlueprintReadWrite, Category = "CineSpline")
	float Aperture;

	UPROPERTY(BlueprintReadWrite, Category = "CineSpline")
	float FocusDistance;
};