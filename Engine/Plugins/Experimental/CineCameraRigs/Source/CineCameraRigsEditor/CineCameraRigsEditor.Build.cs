// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CineCameraRigsEditor : ModuleRules
{
	public CineCameraRigsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
                "ComponentVisualizers",
				"DetailCustomizations",
				"Engine",
				"PlacementMode",
				"SlateCore",
				"Slate",
				"UnrealEd",
				"CineCameraRigs",
				"PropertyEditor",
				"InputCore",
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			});
	}
}
