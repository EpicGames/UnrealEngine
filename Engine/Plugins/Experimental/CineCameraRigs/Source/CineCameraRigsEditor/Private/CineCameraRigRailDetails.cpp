// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraRigRailDetails.h"
#include "CineCameraRigRail.h"
#include "CineSplineComponent.h"
#include "CameraRig_Rail.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailGroup.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "UObject/PropertyIterator.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"

#include "ComponentVisualizer.h"
#include "CineSplineComponentVisualizer.h"

#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"

#include "Algo/AllOf.h"
#include "Algo/NoneOf.h"

#define LOCTEXT_NAMESPACE "FCineCameraRigRailDetails"

void FCineCameraRigRailDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() != 1)
	{
		return;
	}

	ACineCameraRigRail* RigRailActor = Cast<ACineCameraRigRail>(Objects[0].Get());
	if(RigRailActor == nullptr)
	{
		return;
	}
	RigRailActorPtr = RigRailActor;

	CustomizeRailControlCategory(DetailBuilder);
	CustomizeSplineVisualizationCategory(DetailBuilder);
	CustomizeAttachmentCategory(DetailBuilder);
	CustomizeDriveModeCategory(DetailBuilder);
	HideExtraCategories(DetailBuilder);
}

void FCineCameraRigRailDetails::CustomizeRailControlCategory(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> AbsolutePositionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, AbsolutePositionOnRail));
	TSharedRef<IPropertyHandle> UseAbsolutePositionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bUseAbsolutePosition));
	TSharedRef<IPropertyHandle> OrigPositionPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACameraRig_Rail, CurrentPositionOnRail), ACameraRig_Rail::StaticClass());
	TSharedRef<IPropertyHandle> LockOrientationPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACameraRig_Rail, bLockOrientationToRail), ACameraRig_Rail::StaticClass());

	const TAttribute<bool> EditCondition = TAttribute<bool>::Create([this, UseAbsolutePositionPropertyHandle]()
		{
			bool bCond = false;
			UseAbsolutePositionPropertyHandle->GetValue(bCond);
			return !bCond;
		});

	DetailBuilder.EditDefaultProperty(LockOrientationPropertyHandle)->Visibility(EVisibility::Hidden);

	IDetailCategoryBuilder& RailControlsCategory = DetailBuilder.EditCategory("Rail Controls");
	IDetailPropertyRow& UseAbsolutePositionRow = RailControlsCategory.AddProperty(UseAbsolutePositionPropertyHandle);
	IDetailPropertyRow& AbsolutePositionRow = RailControlsCategory.AddProperty(AbsolutePositionPropertyHandle);
	RailControlsCategory.AddProperty(OrigPositionPropertyHandle)
		.EditCondition(EditCondition, nullptr);

	UseAbsolutePositionRow.CustomWidget(false)
		.NameContent()
		[
			UseAbsolutePositionPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([UseAbsolutePositionPropertyHandle]() -> ECheckBoxState
				{
					bool IsChecked;
					UseAbsolutePositionPropertyHandle.Get().GetValue(IsChecked);
					return IsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged(this, &FCineCameraRigRailDetails::OnUseAbsolutePositionChanged, UseAbsolutePositionPropertyHandle)
		];

	AbsolutePositionRow.CustomWidget(false)
		.NameContent()
		[
			AbsolutePositionPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.ToolTipText(LOCTEXT("CurrentPostionToolTip", "Postion property using custom parameterization"))
			.AllowSpin(true)
			.MinSliderValue(this, &FCineCameraRigRailDetails::GetAbsolutePositionSliderMinValue)
			.MaxSliderValue(this, &FCineCameraRigRailDetails::GetAbsolutePositionSliderMaxValue)
			.Value(this, &FCineCameraRigRailDetails::GetAbsolutePosition)
			.OnValueChanged(this, &FCineCameraRigRailDetails::OnAbsolutePositionChanged)
			.OnValueCommitted(this, &FCineCameraRigRailDetails::OnAbsolutePositionCommitted)
			.OnBeginSliderMovement(this, &FCineCameraRigRailDetails::OnBeginAbsolutePositionSliderMovement)
			.OnEndSliderMovement(this, &FCineCameraRigRailDetails::OnEndAbsolutePositionSliderMovement)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

void FCineCameraRigRailDetails::CustomizeSplineVisualizationCategory(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& VisualizationCategory = DetailBuilder.EditCategory("SplineVisualization");
	TSharedRef<IPropertyHandle> ShowRailPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACameraRig_Rail, bShowRailVisualization), ACameraRig_Rail::StaticClass());
	TSharedRef<IPropertyHandle> PreviewScalePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACameraRig_Rail, PreviewMeshScale), ACameraRig_Rail::StaticClass());
	TSharedRef<IPropertyHandle> DisplaySpeedHeatmapPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bDisplaySpeedHeatmap));
	TSharedRef<IPropertyHandle> SampleCountPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, SpeedSampleCountPerSegment));
	VisualizationCategory.AddProperty(ShowRailPropertyHandle);
	VisualizationCategory.AddProperty(DisplaySpeedHeatmapPropertyHandle);
	VisualizationCategory.AddProperty(SampleCountPropertyHandle);
	VisualizationCategory.AddProperty(PreviewScalePropertyHandle);
}

void FCineCameraRigRailDetails::CustomizeAttachmentCategory(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> LockOrientationPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACameraRig_Rail, bLockOrientationToRail), ACameraRig_Rail::StaticClass());
	const TAttribute<bool> RotationEditCondition = TAttribute<bool>::Create([this, LockOrientationPropertyHandle]()
		{
			bool bCond = false;
			LockOrientationPropertyHandle->GetValue(bCond);
			return bCond;
		});

	IDetailCategoryBuilder& AttachmentCategory = DetailBuilder.EditCategory("Attachment");

	// Attachment : Location
	TArray<TSharedRef<IPropertyHandle>> LocationPropertyHandles;
	LocationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachLocationX)));
	LocationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachLocationY)));
	LocationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachLocationZ)));

	IDetailGroup& LocationGroup = AttachmentCategory.AddGroup(TEXT("Location"), LOCTEXT("LocationLabel", "Location"));
	LocationGroup.HeaderRow()
		.NameContent()
			[
			SNew(STextBlock)
			.Text(LOCTEXT("LocationLabel", "Location"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		.ValueContent()
			[
			SNew(SCheckBox)
			.IsChecked(this, &FCineCameraRigRailDetails::IsAttachOptionChecked, LocationPropertyHandles)
			.OnCheckStateChanged(this, &FCineCameraRigRailDetails::OnAttachOptionChanged, LocationPropertyHandles)
			];

	for (auto& PropertyHandle : LocationPropertyHandles)
	{
		DetailBuilder.HideProperty(PropertyHandle);
		LocationGroup.AddPropertyRow(PropertyHandle);
	}


	// Attachment : Rotation
	TArray<TSharedRef<IPropertyHandle>> RotationPropertyHandles;
	RotationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachRotationX)));
	RotationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachRotationY)));
	RotationPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bAttachRotationZ)));

	IDetailGroup& RotationGroup = AttachmentCategory.AddGroup(TEXT("Rotation"), LOCTEXT("RotationLabel", "Rotation"));
	RotationGroup.HeaderRow()
		.EditCondition(RotationEditCondition, nullptr)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RotationLabel", "Rotation"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText_Lambda([LockOrientationPropertyHandle]()
				{
					bool LockOrientation;
					LockOrientationPropertyHandle.Get().GetValue(LockOrientation);
					if (!LockOrientation)
					{
						return FText(LOCTEXT("RotationAttachmentDisabledToolTip", "Disabled because LockOrientationToRail is false"));
					}
					return LOCTEXT("RotationAttachmentToolTip", "Determines if camera mount inherits Rotation");
				})
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FCineCameraRigRailDetails::IsAttachRotationOptionChecked, RotationPropertyHandles)
			.OnCheckStateChanged(this, &FCineCameraRigRailDetails::OnAttachOptionChanged, RotationPropertyHandles)
		];

	for (auto& PropertyHandle : RotationPropertyHandles)
	{
		DetailBuilder.HideProperty(PropertyHandle);
		IDetailPropertyRow& RotationAxisRow = RotationGroup.AddPropertyRow(PropertyHandle);
		RotationAxisRow.EditCondition(RotationEditCondition, nullptr);

		RotationAxisRow.CustomWidget(false)
			.NameContent()
			[
				SNew(STextBlock)
					.Text(PropertyHandle->GetPropertyDisplayName())
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText_Lambda([PropertyHandle, LockOrientationPropertyHandle]()
						{
							bool LockOrientation;
							LockOrientationPropertyHandle.Get().GetValue(LockOrientation);
							if (!LockOrientation)
							{
								return FText(LOCTEXT("RotationAxisAttachmentToolTip", "Disabled because LockOrientationToRail is false"));
							}
							return PropertyHandle->GetToolTipText();
						})
			]
			.ValueContent()
			.MinDesiredWidth(125.0f)
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([PropertyHandle, LockOrientationPropertyHandle]() -> ECheckBoxState
						{
							bool IsChecked;
							PropertyHandle.Get().GetValue(IsChecked);
							bool LockOrientation;
							LockOrientationPropertyHandle.Get().GetValue(LockOrientation);
							return IsChecked && LockOrientation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged(this, &FCineCameraRigRailDetails::OnRotationAxisAttachmentChanged, PropertyHandle)
			];
	}

	// Attachment : Camera
	TArray<TSharedRef<IPropertyHandle>> CameraPropertyHandles;
	CameraPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bInheritFocalLength)));
	CameraPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bInheritAperture)));
	CameraPropertyHandles.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bInheritFocusDistance)));

	IDetailGroup& CameraGroup = AttachmentCategory.AddGroup(TEXT("Camera"), LOCTEXT("CameraLabel", "Camera"));
	CameraGroup.HeaderRow()
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CameraLabel", "Camera"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FCineCameraRigRailDetails::IsAttachOptionChecked, CameraPropertyHandles)
			.OnCheckStateChanged(this, &FCineCameraRigRailDetails::OnAttachOptionChanged, CameraPropertyHandles)
		];

	for (auto& PropertyHandle : CameraPropertyHandles)
	{
		DetailBuilder.HideProperty(PropertyHandle);
		CameraGroup.AddPropertyRow(PropertyHandle);
	}
}

void FCineCameraRigRailDetails::CustomizeDriveModeCategory(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& DriveModeCategory = DetailBuilder.EditCategory("DriveMode");
	DriveModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, DriveMode));
	TSharedPtr<IPropertyHandle> DurationHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USplineComponent, Duration), USplineComponent::StaticClass());
	TSharedPtr<IPropertyHandle> SpeedHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, Speed));
	TSharedPtr<IPropertyHandle> PlayHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bPlay));
	TSharedPtr<IPropertyHandle> LoopHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bLoop));
	TSharedPtr<IPropertyHandle> ReverseHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, bReverse));

	DriveModeCategory.AddProperty(DriveModeHandle.ToSharedRef());

	DriveModeCategory.AddProperty(DurationHandle.ToSharedRef())
		.EditCondition(TAttribute<bool>::Create([this]() { return GetDriveMode() == ECineCameraRigRailDriveMode::Duration; }), nullptr);
	DriveModeCategory.AddProperty(SpeedHandle.ToSharedRef())
		.EditCondition(TAttribute<bool>::Create([this]() { return GetDriveMode() == ECineCameraRigRailDriveMode::Speed; }), nullptr);
	DriveModeCategory.AddProperty(PlayHandle.ToSharedRef())
		.EditCondition(TAttribute<bool>::Create([this]() { return GetDriveMode() != ECineCameraRigRailDriveMode::Manual; }), nullptr);
	DriveModeCategory.AddProperty(LoopHandle.ToSharedRef())
		.EditCondition(TAttribute<bool>::Create([this]() { return GetDriveMode() != ECineCameraRigRailDriveMode::Manual; }), nullptr);
	DriveModeCategory.AddProperty(ReverseHandle.ToSharedRef())
		.EditCondition(TAttribute<bool>::Create([this]() { return GetDriveMode() != ECineCameraRigRailDriveMode::Manual; }), nullptr);

}

void FCineCameraRigRailDetails::HideExtraCategories(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("Editor");
	DetailBuilder.HideCategory("Selected Points");
	DetailBuilder.HideCategory("HLOD");
	DetailBuilder.HideCategory("PathTracing");
	DetailBuilder.HideCategory("Spline");
	DetailBuilder.HideCategory("Navigation");
	DetailBuilder.HideCategory("Tags");
	DetailBuilder.HideCategory("Cooking");
	DetailBuilder.HideCategory("LOD");
	DetailBuilder.HideCategory("TextureStreaming");
	DetailBuilder.HideCategory("RayTracing");
	DetailBuilder.HideCategory("AssetUserData");
}

ECineCameraRigRailDriveMode FCineCameraRigRailDetails::GetDriveMode() const
{
	if (!DriveModeHandle.IsValid())
	{
		return ECineCameraRigRailDriveMode::Manual;
	}
	uint8 DriveModeValue;
	DriveModeHandle->GetValue(DriveModeValue);
	return static_cast<ECineCameraRigRailDriveMode>(DriveModeValue);
}

ECheckBoxState FCineCameraRigRailDetails::IsAttachOptionChecked(TArray<TSharedRef<IPropertyHandle>> PropertyHandles) const
{
	const bool bAllChecked = Algo::AllOf(PropertyHandles, [](const TSharedRef<IPropertyHandle>& PropertyHandle)
		{
			bool bValue;
			PropertyHandle->GetValue(bValue);
			return bValue;
		});
	const bool bNoneChecked = Algo::NoneOf(PropertyHandles, [](const TSharedRef<IPropertyHandle>& PropertyHandle)
		{
			bool bValue;
			PropertyHandle->GetValue(bValue);
			return bValue;
		});

	if (bAllChecked)
	{
		return ECheckBoxState::Checked;
	}
	else if (bNoneChecked)
	{
		return ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

ECheckBoxState FCineCameraRigRailDetails::IsAttachRotationOptionChecked(TArray<TSharedRef<IPropertyHandle>> PropertyHandles) const
{
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		if (!RigRailActor->bLockOrientationToRail)
		{
			return ECheckBoxState::Unchecked;
		}
	}
	return IsAttachOptionChecked(PropertyHandles);
}

void FCineCameraRigRailDetails::OnAttachOptionChanged(ECheckBoxState NewState, TArray<TSharedRef<IPropertyHandle>> PropertyHandles)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	const bool bValue = (NewState == ECheckBoxState::Checked) ? true : false;
	for (TSharedRef<IPropertyHandle>& PropertyHandle : PropertyHandles)
	{
		PropertyHandle->SetValue(bValue);
	}
}

void FCineCameraRigRailDetails::OnAbsolutePositionChanged(float NewValue)
{
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		RigRailActor->AbsolutePositionOnRail = NewValue;
		static FProperty* AbsolutePositionProperty = FindFProperty<FProperty>(ACineCameraRigRail::StaticClass(), GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, AbsolutePositionOnRail));
		FPropertyChangedEvent SetValueEvent(AbsolutePositionProperty);
		RigRailActor->PostEditChangeProperty(SetValueEvent);
		if (GEditor)
		{
			GEditor->RedrawLevelEditingViewports(true);
		}
	}
}

void FCineCameraRigRailDetails::OnUseAbsolutePositionChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}
	const bool bValue = (NewState == ECheckBoxState::Checked) ? true : false;
	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->SetValue(bValue);
	}
	
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		if (UCineSplineComponent* SplineComp = RigRailActor->GetCineSplineComponent())
		{
			SplineComp->bShouldVisualizeNormalizedPosition = !bValue;
		}
	}
}

void FCineCameraRigRailDetails::OnAbsolutePositionCommitted(float NewValue, ETextCommit::Type CommitType)
{
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetAbsolutePosition", "Set rig rail custom position"));
		RigRailActor->SetFlags(RF_Transactional);
		RigRailActor->Modify();
		RigRailActor->AbsolutePositionOnRail = NewValue;
		static FProperty* AbsolutePositionProperty = FindFProperty<FProperty>(ACineCameraRigRail::StaticClass(), GET_MEMBER_NAME_CHECKED(ACineCameraRigRail, AbsolutePositionOnRail));
		FPropertyChangedEvent SetValueEvent(AbsolutePositionProperty);
		RigRailActor->PostEditChangeProperty(SetValueEvent);
		if (GEditor)
		{
			GEditor->RedrawLevelEditingViewports(true);
		}
	}
}

void FCineCameraRigRailDetails::OnBeginAbsolutePositionSliderMovement()
{
	if (!bAbsolutePositionSliderStartedTransaction)
	{
		if (GEditor)
		{
			bAbsolutePositionSliderStartedTransaction = true;
			GEditor->BeginTransaction(LOCTEXT("AbsolutePositionSliderTransaction", "Set rig rail custom position via slider"));
			if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
			{
				RigRailActor->SetFlags(RF_Transactional);
				RigRailActor->Modify();
			}
		}
	}
}

void FCineCameraRigRailDetails::OnEndAbsolutePositionSliderMovement(float NewValue)
{
	if (bAbsolutePositionSliderStartedTransaction)
	{
		if (GEditor)
		{
			GEditor->EndTransaction();
			bAbsolutePositionSliderStartedTransaction = false;
		}
	}
}

TOptional<float> FCineCameraRigRailDetails::GetAbsolutePosition() const
{
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		return RigRailActor->AbsolutePositionOnRail;
	}
	return 0.0f;
}

TOptional<float> FCineCameraRigRailDetails::GetAbsolutePositionSliderMinValue() const
{

	float MinValue = 1.0f;
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		UCineSplineComponent* SplineComp = RigRailActor->GetCineSplineComponent();
		const UCineSplineMetadata* MetaData = Cast<UCineSplineMetadata>(SplineComp->GetSplinePointsMetadata());
		MinValue = MetaData->AbsolutePosition.Points[0].OutVal;
	}
	return MinValue;
}

TOptional<float> FCineCameraRigRailDetails::GetAbsolutePositionSliderMaxValue() const
{

	float MaxValue = 5.0f;
	if (ACineCameraRigRail* RigRailActor = RigRailActorPtr.Get())
	{
		UCineSplineComponent* SplineComp = RigRailActor->GetCineSplineComponent();
		const UCineSplineMetadata* MetaData = Cast<UCineSplineMetadata>(SplineComp->GetSplinePointsMetadata());
		int32 NumPoints = MetaData->AbsolutePosition.Points.Num();
		MaxValue = MetaData->AbsolutePosition.Points[NumPoints - 1].OutVal;
	}
	return MaxValue;
}

void FCineCameraRigRailDetails::OnRotationAxisAttachmentChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (NewState == ECheckBoxState::Undetermined)
	{
		return;
	}
	const bool bValue = (NewState == ECheckBoxState::Checked) ? true : false;
	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->SetValue(bValue);
	}
}
#undef LOCTEXT_NAMESPACE