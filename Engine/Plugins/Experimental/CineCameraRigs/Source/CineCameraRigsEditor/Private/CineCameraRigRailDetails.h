// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Styling/SlateTypes.h"

#include "CineCameraRigRail.h"

class IPropertyHandle;

namespace ETextCommit { enum Type : int; }

/**
 * Implements a details view customization for the ACineCameraRigRail
 */
class FCineCameraRigRailDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FCineCameraRigRailDetails>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	TOptional<float> CurrentPositionValue = 1.0f;

private:
	TWeakObjectPtr<ACineCameraRigRail> RigRailActorPtr;

	TOptional<float> GetAbsolutePosition() const;

	TOptional<float> GetAbsolutePositionSliderMinValue() const;
	TOptional<float> GetAbsolutePositionSliderMaxValue() const;
	void OnBeginAbsolutePositionSliderMovement();
	void OnEndAbsolutePositionSliderMovement(float NewValue);
	void OnAbsolutePositionCommitted(float NewValue, ETextCommit::Type CommitType);
	void OnAbsolutePositionChanged(float NewValue);

	bool bAbsolutePositionSliderStartedTransaction = false;

	ECheckBoxState IsAttachOptionChecked(TArray<TSharedRef<IPropertyHandle>> PropertyHandles) const;
	void OnAttachOptionChanged(ECheckBoxState NewState, TArray<TSharedRef<IPropertyHandle>> PropertyHandles);
	
	TSharedPtr<IPropertyHandle> DriveModeHandle;
	ECineCameraRigRailDriveMode GetDriveMode() const;

	void OnUseAbsolutePositionChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle);

	void CustomizeRailControlCategory(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeSplineVisualizationCategory(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeAttachmentCategory(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeDriveModeCategory(IDetailLayoutBuilder& DetailBuilder);
	void HideExtraCategories(IDetailLayoutBuilder& DetailBuilder);

	ECheckBoxState IsAttachRotationOptionChecked(TArray<TSharedRef<IPropertyHandle>> PropertyHandles) const;
	void OnRotationAxisAttachmentChanged(ECheckBoxState NewState, TSharedRef<IPropertyHandle> PropertyHandle);
};
