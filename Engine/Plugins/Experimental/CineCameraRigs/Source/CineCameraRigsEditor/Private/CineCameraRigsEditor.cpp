// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "CineSplineComponentVisualizer.h"
#include "CineSplineComponent.h"
#include "CineCameraRigRail.h"
#include "CineCameraRigRailDetails.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "IPlacementModeModule.h"
#include "UnrealEdGlobals.h"
#include "PropertyEditorModule.h"
#include "PropertyEditorDelegates.h"

#define LOCTEXT_NAMESPACE "CineCameraRigsEditor"

class FCineCameraRigsEditorModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		CineSplineComponentName = UCineSplineComponent::StaticClass()->GetFName();

		if (GUnrealEd)
		{
			TSharedPtr<FCineSplineComponentVisualizer> Visualizer = MakeShared<FCineSplineComponentVisualizer>();
			GUnrealEd->RegisterComponentVisualizer(CineSplineComponentName, Visualizer);
			Visualizer->OnRegister();
			RegisterCustomizations();
			RegisterPlacementModeItems();
		}
	}

	virtual void ShutdownModule() override
	{
		if (GUnrealEd)
		{
			GUnrealEd->UnregisterComponentVisualizer(CineSplineComponentName);
			if (UObjectInitialized() && !IsEngineExitRequested())
			{
				UnregisterCustomizations();
			}
		}
	}

private:
	FName CineSplineComponentName;


	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(ACineCameraRigRail::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FCineCameraRigRailDetails::MakeInstance));
	}
	void UnregisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(ACineCameraRigRail::StaticClass()->GetFName());
	}
	void RegisterPlacementModeItems()
	{
		if (const FPlacementCategoryInfo* Info = GetCinematicPlacementCategoryInfo())
		{
			IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
				*ACineCameraRigRail::StaticClass(),
				FAssetData(ACineCameraRigRail::StaticClass())));
		}
	}

	const FPlacementCategoryInfo* GetCinematicPlacementCategoryInfo() const
	{
		IPlacementModeModule& PlacmentModeModule = IPlacementModeModule::Get();

		if (const FPlacementCategoryInfo* RegisteredInfo = PlacmentModeModule.GetRegisteredPlacementCategory("Cinematic"))
		{
			return RegisteredInfo;
		}

		FPlacementCategoryInfo Info(
			LOCTEXT("CinematicCategoryName", "Cinematic"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Cinematics"),
			"Cinematic",
			TEXT("PMCinematic"),
			25
		);
		Info.ShortDisplayName = LOCTEXT("CinematicShortCategoryName", "Cine");

		PlacmentModeModule.RegisterPlacementCategory(Info);
		return PlacmentModeModule.GetRegisteredPlacementCategory("Cinematic");
	}
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FCineCameraRigsEditorModule, CineCameraRigsEditor)
