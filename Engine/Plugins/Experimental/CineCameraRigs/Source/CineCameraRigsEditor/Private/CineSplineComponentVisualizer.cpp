// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineSplineComponentVisualizer.h"
#include "CineSplineComponent.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "EditorViewportClient.h"
#include "Engine/Engine.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet/KismetMathLibrary.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"

#define LOCTEXT_NAMESPACE "SplineComponentVisualizer"

class FCineSplineComponentVisualizerCommands : public TCommands< FCineSplineComponentVisualizerCommands>
{
public:
	FCineSplineComponentVisualizerCommands() : TCommands< FCineSplineComponentVisualizerCommands>
		(
			"CineSplineComponentVisualizer",
			LOCTEXT("CineSplineComponentVisualizer", "CineSpline Component Visualizer"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
			)
	{}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(VisualizeNormalizedPosition, "Show Normalized Position Value", "Whether the visualization should show normalized position or absolute position.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(VisualizeSplineLength, "Show Spline Length", "Whether the visualization should show spline length.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(VisualizePointRotation, "Show Point Rotation", "Whether the visualization should show point rotation arrows.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ManipulatePointRotation, "Manipulate Point Rotation", "Use rotation manipulator to edit PointRotation metadata.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(SetMetadataToLinear, "Linear", "Set spline point to Linear type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetMetadataToCurve, "Curve", "Set spline point to Curve type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetMetadataToConstant, "Constant", "Set spline point to Constant type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetMetadataToCurveClamped, "CurveClamped", "Set spline point to CurveClamped type", EUserInterfaceActionType::RadioButton, FInputChord());
	}

public:

	TSharedPtr<FUICommandInfo> VisualizeNormalizedPosition;
	TSharedPtr<FUICommandInfo> VisualizeSplineLength;
	TSharedPtr<FUICommandInfo> VisualizePointRotation;
	TSharedPtr<FUICommandInfo> ManipulatePointRotation;
	TSharedPtr<FUICommandInfo> SetMetadataToLinear;
	TSharedPtr<FUICommandInfo> SetMetadataToCurve;
	TSharedPtr<FUICommandInfo> SetMetadataToConstant;
	TSharedPtr<FUICommandInfo> SetMetadataToCurveClamped;
};

FCineSplineComponentVisualizer::FCineSplineComponentVisualizer()
	: FSplineComponentVisualizer()
{
	FCineSplineComponentVisualizerCommands::Register();
	CineSplineComponentVisualizerActions = MakeShareable(new FUICommandList);
}

void FCineSplineComponentVisualizer::OnRegister()
{
	FSplineComponentVisualizer::OnRegister();

	const auto& Commands = FCineSplineComponentVisualizerCommands::Get();

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeNormalizedPosition,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetVisualizeNormalizedPosition),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsVisualizeNormalizedPosition)
	);

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeSplineLength,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetVisualizeSplineLength),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsVisualizeSplineLength)
	);

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizePointRotation,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetVisualizePointRotation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsVisualizePointRotation)
	);

	SplineComponentVisualizerActions->MapAction(
		Commands.ManipulatePointRotation,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetManipulatePointRotation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsManipulatePointRotation)
	);

	SplineComponentVisualizerActions->MapAction(
		Commands.SetMetadataToLinear,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetMetadataTangent, ECineSplineMetadataTangentType::Linear),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsMetadataTangent, ECineSplineMetadataTangentType::Linear)
	);
	SplineComponentVisualizerActions->MapAction(
		Commands.SetMetadataToCurve,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetMetadataTangent, ECineSplineMetadataTangentType::Curve),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsMetadataTangent, ECineSplineMetadataTangentType::Curve)
	);
	SplineComponentVisualizerActions->MapAction(
		Commands.SetMetadataToConstant,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetMetadataTangent, ECineSplineMetadataTangentType::Constant),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsMetadataTangent, ECineSplineMetadataTangentType::Constant)
	);
	SplineComponentVisualizerActions->MapAction(
		Commands.SetMetadataToCurveClamped,
		FExecuteAction::CreateSP(this, &FCineSplineComponentVisualizer::OnSetMetadataTangent, ECineSplineMetadataTangentType::CurveClamped),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCineSplineComponentVisualizer::IsMetadataTangent, ECineSplineMetadataTangentType::CurveClamped)
	);
}

FCineSplineComponentVisualizer::~FCineSplineComponentVisualizer()
{
	FCineSplineComponentVisualizerCommands::Unregister();
}

void FCineSplineComponentVisualizer::DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FSplineComponentVisualizer::DrawVisualizationHUD(Component, Viewport, View, Canvas);
	if (Canvas == nullptr || View == nullptr)
	{
		return;
	}

	const FIntRect CanvasRect = Canvas->GetViewRect();
	const float HalfX = CanvasRect.Width() / 2.f;
	const float HalfY = CanvasRect.Height() / 2.f;
	const float TextPositionOffset = 12.0f;

	if (const UCineSplineComponent* SplineComp = Cast<const UCineSplineComponent>(Component))
	{
		if (UCineSplineMetadata* Metadata = SplineComp->CineSplineMetadata)
		{
			const int32 NumOfPoints = SplineComp->GetNumberOfSplinePoints();
			for (int32 i = 0; i < NumOfPoints; ++i)
			{
				FVector Location = SplineComp->GetLocationAtSplinePoint(i, ESplineCoordinateSpace::World);
				FPlane Plane(0, 0, 0, 0);
				Plane = View->Project(Location);
				if (Plane.Z < 0.0)
				{
					continue;
				}
				const FVector Position(Plane);
				const float DrawPositionX = FMath::FloorToFloat(HalfX + Position.X * HalfX);
				const float DrawPositionY = FMath::FloorToFloat(HalfY + -1.f * Position.Y * HalfY);
				
				FNumberFormattingOptions FmtOptions;
				FmtOptions.SetMaximumFractionalDigits(4);
				FmtOptions.SetMinimumFractionalDigits(1);

				// Draw normalized position values or AbsolutePosition metadata values, based on bUseAbsolutePosition parameter in CineCameraRigRailActor
				const bool bShowNormalizedPosition = SplineComp->bShouldVisualizeNormalizedPosition;
				const float Value = bShowNormalizedPosition ? UKismetMathLibrary::SafeDivide(SplineComp->GetDistanceAlongSplineAtSplinePoint(i), SplineComp->GetSplineLength()) : Metadata->AbsolutePosition.Points[i].OutVal;
				const FLinearColor Color = bShowNormalizedPosition ? FLinearColor(0, 1, 1, 1) : FMath::IsNearlyEqual(FMath::Frac(Metadata->AbsolutePosition.Points[i].OutVal), 0.0f) ? FLinearColor(1, 1, 0, 1) : FLinearColor(0, 1, 1, 1);
				const FText Text = FText::AsNumber(Value , &FmtOptions);
				Canvas->DrawShadowedString(DrawPositionX, DrawPositionY, *Text.ToString(), GEngine->GetLargeFont(), Color, FLinearColor::Black);

				if (SplineComp->bShouldVisualizeSplineLength)
				{
					const float Distance = SplineComp->GetDistanceAlongSplineAtSplinePoint(i);
					const FText DistanceText = FText::AsNumber(Distance, &FmtOptions);
					Canvas->DrawShadowedString(DrawPositionX, (DrawPositionY+TextPositionOffset), *DistanceText.ToString(), GEngine->GetLargeFont(), FLinearColor::White, FLinearColor::Black);
				}
			}
		}
	}
}


void FCineSplineComponentVisualizer::DrawVisualization(
	const UActorComponent* Component,
	const FSceneView* View,
	FPrimitiveDrawInterface* PDI
)
{
	FSplineComponentVisualizer::DrawVisualization(Component, View, PDI);

	const UCineSplineComponent* CineSplineComponent = Cast<const UCineSplineComponent>(Component);

	if (CineSplineComponent && CineSplineComponent->bShouldVisualizePointRotation)
	{
		const int32 NumSplinePoints = CineSplineComponent->GetNumberOfSplinePoints();
		const float ArrowLength = 100.0f;
		const float ArrowSize = 5.0f;
		const FColor ArrowColor = FColor::Cyan;

		for (int32 PointIdx = 0; PointIdx < NumSplinePoints; PointIdx++)
		{
			const FVector Location = CineSplineComponent->GetLocationAtSplinePoint(PointIdx, ESplineCoordinateSpace::World);
			const FQuat Rotation = CineSplineComponent->GetPointRotationAtSplinePoint(PointIdx);

			const FVector ArrowDirection = Rotation.RotateVector(FVector::ForwardVector);
			const FVector ArrowEndLocation = Location + ArrowDirection * ArrowLength;
			const FTransform Transform = FTransform(Rotation, Location);

			PDI->DrawLine(Location, ArrowEndLocation, ArrowColor, SDPG_World, 1.0f, 0.0f, true);
			DrawDirectionalArrow(PDI, Transform.ToMatrixNoScale(), ArrowColor, ArrowLength, ArrowSize, SDPG_World);
		}
	}
}

void FCineSplineComponentVisualizer::GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const
{
	FSplineComponentVisualizer::GenerateContextMenuSections(InMenuBuilder);
	if (GetEditedSplineComponent() != nullptr)
	{
		InMenuBuilder.BeginSection("CineSpline", LOCTEXT("CineSpline", "CineSpline"));
		{
			InMenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().VisualizeNormalizedPosition);
			InMenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().VisualizeSplineLength);
			InMenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().VisualizePointRotation);
			InMenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().ManipulatePointRotation);
			InMenuBuilder.AddSubMenu(
				LOCTEXT("CineSplineMetadataTangentType", "Metadata Tangent Type"),
				LOCTEXT("CineSplineMetadataTangentTypeTooltip", "Define the type of the metadata tangent"),
				FNewMenuDelegate::CreateSP(this, &FCineSplineComponentVisualizer::GenerateMetadataTangentTypeSubMenu)
			);
		}
		InMenuBuilder.EndSection();
	}
}

bool FCineSplineComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale)
{
	if (!DeltaRotate.IsZero() && bManipulatePointRotation)
	{
		return TransformPointRotation(DeltaRotate);
	}
	else
	{
		return FSplineComponentVisualizer::HandleInputDelta(ViewportClient, Viewport, DeltaTranslate, DeltaRotate, DeltaScale);
	}
}

bool FCineSplineComponentVisualizer::TransformPointRotation(const FRotator& DeltaRotate)
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		check(SelectionState);

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(NumPoints);

		check(SelectedKeys.Num() > 0);
		check(SelectedKeys.Contains(LastKeyIndexSelected));

		SplineComp->Modify();

		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			check(SelectedKeyIndex >= 0);
			check(SelectedKeyIndex < NumPoints);

			const FQuat Rotation = SplineComp->GetPointRotationAtSplinePoint(SelectedKeyIndex);
			SplineComp->SetPointRotationAtSplinePoint(SelectedKeyIndex, DeltaRotate.Quaternion() * Rotation);
		}

		return true;
	
	}
	return false;
}

bool FCineSplineComponentVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	if (bManipulatePointRotation && (ViewportClient->GetWidgetCoordSystemSpace() == COORD_Local || ViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate))
	{
		OutMatrix = FRotationMatrix(FRotator());
		return true;
	}
	return FSplineComponentVisualizer::GetCustomInputCoordinateSystem(ViewportClient, OutMatrix);
}

void FCineSplineComponentVisualizer::OnSetVisualizeNormalizedPosition()
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		SplineComp->bShouldVisualizeNormalizedPosition = !SplineComp->bShouldVisualizeNormalizedPosition;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FCineSplineComponentVisualizer::IsVisualizeNormalizedPosition() const
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		return SplineComp->bShouldVisualizeNormalizedPosition;
	}
	return false;
}

void FCineSplineComponentVisualizer::OnSetVisualizeSplineLength()
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		SplineComp->bShouldVisualizeSplineLength = !SplineComp->bShouldVisualizeSplineLength;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FCineSplineComponentVisualizer::IsVisualizeSplineLength() const
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		return SplineComp->bShouldVisualizeSplineLength;
	}
	return false;
}

void FCineSplineComponentVisualizer::OnSetVisualizePointRotation()
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		SplineComp->bShouldVisualizePointRotation = !SplineComp->bShouldVisualizePointRotation;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FCineSplineComponentVisualizer::IsVisualizePointRotation() const
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		return SplineComp->bShouldVisualizePointRotation;
	}
	return false;
}

void FCineSplineComponentVisualizer::OnSetManipulatePointRotation()
{
	bManipulatePointRotation = !bManipulatePointRotation;
}

bool FCineSplineComponentVisualizer::IsManipulatePointRotation() const
{
	return bManipulatePointRotation;
}

void FCineSplineComponentVisualizer::GenerateMetadataTangentTypeSubMenu(FMenuBuilder& MenuBuilder) const
{
	if (const UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		MenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().SetMetadataToLinear);
		MenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().SetMetadataToCurve);
		MenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().SetMetadataToConstant);
		MenuBuilder.AddMenuEntry(FCineSplineComponentVisualizerCommands::Get().SetMetadataToCurveClamped);
	}
}

void FCineSplineComponentVisualizer::OnSetMetadataTangent(ECineSplineMetadataTangentType TangentType)
{
	if (UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent()))
	{
		check(SelectionState);
		const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
		for (int32 SelectedKeyIndex : SelectedKeys)
		{
			check(SelectedKeyIndex >= 0);
			check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());
			SplineComp->CineSplineMetadata->SetTangentType(SelectedKeyIndex, TangentType);
		}
		SplineComp->CineSplineMetadata->AutoSetTangents();
	}
}

bool FCineSplineComponentVisualizer::IsMetadataTangent(ECineSplineMetadataTangentType TangentType)
{
	if (IsKeySelectionValid())
	{
		UCineSplineComponent* SplineComp = Cast<UCineSplineComponent>(GetEditedSplineComponent());
		check(SplineComp != nullptr);
		check(SelectionState);

		EInterpCurveMode Mode = ConvertCineSplineMetadataTangentTypeToInterpCurveMode(TangentType);
		if(Mode != CIM_Unknown)
		{
			const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
			for (int32 SelectedKeyIndex : SelectedKeys)
			{
				check(SelectedKeyIndex >= 0);
				check(SelectedKeyIndex < SplineComp->GetNumberOfSplinePoints());
				const auto& SelectedPoint = SplineComp->CineSplineMetadata->FocalLength.Points[SelectedKeyIndex];
				if (SelectedPoint.InterpMode == Mode)
				{
					return true;
				}
			}
		}
	}

	return false;
}


#undef LOCTEXT_NAMESPACE