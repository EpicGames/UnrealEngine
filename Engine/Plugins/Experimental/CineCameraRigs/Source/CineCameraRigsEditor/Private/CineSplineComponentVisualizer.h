// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HitProxies.h"
#include "InputCoreTypes.h"

#include "ComponentVisualizer.h"
#include "SplineComponentVisualizer.h"

#include "Components/SplineComponent.h"
#include "CineSplineComponent.h"
#include "CineSplineMetadata.h"

class FUICommandList;

class FCineSplineComponentVisualizer : public FSplineComponentVisualizer
{
public:
	FCineSplineComponentVisualizer();
	virtual ~FCineSplineComponentVisualizer();

	//~ Begin FComponentVisualizer Interface
	virtual void OnRegister() override;
	virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const;
	virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;
	virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const override;

protected:
	bool bManipulatePointRotation;

	bool TransformPointRotation(const FRotator& DeltaRotate);

	void OnSetVisualizeNormalizedPosition();
	bool IsVisualizeNormalizedPosition() const;
	void OnSetVisualizeSplineLength();
	bool IsVisualizeSplineLength() const;
	void OnSetVisualizePointRotation();
	bool IsVisualizePointRotation() const;
	void OnSetManipulatePointRotation();
	bool IsManipulatePointRotation() const;

	void OnSetMetadataTangent(ECineSplineMetadataTangentType TangentType);
	bool IsMetadataTangent(ECineSplineMetadataTangentType TangentType);
	void GenerateMetadataTangentTypeSubMenu(FMenuBuilder& MenuBuilder) const;

	TSharedPtr<FUICommandList> CineSplineComponentVisualizerActions;

};