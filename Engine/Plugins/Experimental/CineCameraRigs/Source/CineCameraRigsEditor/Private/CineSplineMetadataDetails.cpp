// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineSplineMetadataDetails.h"
#include "CineSplineMetadata.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Internationalization/Internationalization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "ScopedTransaction.h"
#include "ComponentVisualizer.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "FCineSplineMetadataDetails"

UClass* UCineSplineMetadataDetailsFactory::GetMetadataClass() const
{
	return UCineSplineMetadata::StaticClass();
}

TSharedPtr<ISplineMetadataDetails> UCineSplineMetadataDetailsFactory::Create()
{
	return MakeShared<FCineSplineMetadataDetails>();
}

FText FCineSplineMetadataDetails::GetDisplayName() const
{
	return LOCTEXT("CineSplineMetadataDetails", "CineSplinePointData");
}

template<class T>
bool UpdateMultipleValue(TOptional<T>& CurrentValue, T InValue)
{
	if (!CurrentValue.IsSet())
	{
		CurrentValue = InValue;
	}
	else if (CurrentValue.IsSet() && CurrentValue.GetValue() != InValue)
	{
		CurrentValue.Reset();
		return false;
	}

	return true;
}

void FCineSplineMetadataDetails::Update(USplineComponent* InSplineComponent, const TSet<int32>& InSelectedKeys)
{
	SplineComp = InSplineComponent;
	SelectedKeys = InSelectedKeys;
	AbsolutePositionValue.Reset();
	FocalLengthValue.Reset();
	ApertureValue.Reset();
	FocusDistanceValue.Reset();
	PointRotationValue.Reset();
	InterpTypeValue.Reset();

	if (!IsValid(InSplineComponent))
	{
		return;
	}
	UCineSplineMetadata* Metadata = Cast<UCineSplineMetadata>(InSplineComponent->GetSplinePointsMetadata());
	if (!Metadata)
	{
		return;
	}

	bool bUpdateAbsolutePosition = true;
	bool bUpdateFocalLength = true;
	bool bUpdateAperture = true;
	bool bUpdateFocusDistance = true;
	bool bUpdatePointRotation = true;
	bool bUpdateInterpType = true;

	for (const int32 Index : InSelectedKeys)
	{
		bUpdateAbsolutePosition = bUpdateAbsolutePosition ? UpdateMultipleValue(AbsolutePositionValue, Metadata->AbsolutePosition.Points[Index].OutVal) : false;
		bUpdateFocalLength = bUpdateFocalLength ? UpdateMultipleValue(FocalLengthValue, Metadata->FocalLength.Points[Index].OutVal) : false;
		bUpdateAperture = bUpdateAperture ? UpdateMultipleValue(ApertureValue, Metadata->Aperture.Points[Index].OutVal) : false;
		bUpdateFocusDistance = bUpdateFocusDistance ? UpdateMultipleValue(FocusDistanceValue, Metadata->FocusDistance.Points[Index].OutVal) : false;
		bUpdatePointRotation = bUpdatePointRotation ? UpdateMultipleValue(PointRotationValue, Metadata->PointRotation.Points[Index].OutVal) : false;
		bUpdateInterpType = bUpdateInterpType ? UpdateMultipleValue(InterpTypeValue, ConvertInterpCurveModeToCineSplineMetadataTangentType(Metadata->PointRotation.Points[Index].InterpMode)) : false;
	}
}

template<class T>
void SetValues(FCineSplineMetadataDetails& Details, TArray<FInterpCurvePoint<T>>& Points, const T& NewValue)
{
	const FScopedTransaction Transaction(LOCTEXT("EditSplinePointsMetadata", "Edit spline point metadata"), (Details.SliderEnterCount == 0) );
	Details.SplineComp->GetSplinePointsMetadata()->Modify();
	for (int32 Index : Details.SelectedKeys)
	{
		Points[Index].OutVal = NewValue;
	}

	Details.SplineComp->UpdateSpline();
	Details.SplineComp->bSplineHasBeenEdited = true;

	static TArray<FProperty*> SplineProperties;
	if (SplineProperties.IsEmpty())
	{
		for (const FName& Property : USplineComponent::GetSplinePropertyNames())
		{
			SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
		}
	}

	FComponentVisualizer::NotifyPropertiesModified(Details.SplineComp, SplineProperties);
	Details.Update(Details.SplineComp, Details.SelectedKeys);

	if (GEditor)
	{
		GEditor->RedrawLevelEditingViewports(true);
	}
}

void FCineSplineMetadataDetails::OnBeginSliderMovement()
{
	if (SliderEnterCount == 0)
	{
		GEditor->BeginTransaction(LOCTEXT("EditCineSplineProperty", "Edit CineSpline Property"));
	}
	SliderEnterCount++;
	SplineComp->Modify();
}

void FCineSplineMetadataDetails::OnEndSliderMovement(float NewValue)
{
	SliderEnterCount--;
	check(SliderEnterCount >= 0);
	if (SliderEnterCount == 0)
	{
		GEditor->EndTransaction();
	}
}

UCineSplineMetadata* FCineSplineMetadataDetails::GetMetadata() const
{
	return SplineComp ? Cast<UCineSplineMetadata>(SplineComp->GetSplinePointsMetadata()) : nullptr;
}

void FCineSplineMetadataDetails::OnSetAbsolutePosition(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		SetValues<float>(*this, Metadata->AbsolutePosition.Points, NewValue);
	}
}

void FCineSplineMetadataDetails::OnSetFocalLength(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		SetValues<float>(*this, Metadata->FocalLength.Points, NewValue);
	}
}

void FCineSplineMetadataDetails::OnSetAperture(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		SetValues<float>(*this, Metadata->Aperture.Points, NewValue);
	}
}

void FCineSplineMetadataDetails::OnSetFocusDistance(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		SetValues<float>(*this, Metadata->FocusDistance.Points, NewValue);
	}
}

TOptional<float> FCineSplineMetadataDetails::GetRotation(EAxis::Type Axis) const
{
	return static_cast<float>(PointRotationValue.Get(FQuat::Identity).Rotator().GetComponentForAxis(Axis));
}
void FCineSplineMetadataDetails::OnSetRotation(float NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		FRotator CurrentRotator(PointRotationValue.Get(FQuat::Identity));
		switch (Axis)
		{
		case EAxis::X: CurrentRotator.Roll = NewValue; break;
		case EAxis::Y: CurrentRotator.Pitch = NewValue; break;
		case EAxis::Z: CurrentRotator.Yaw = NewValue; break;
		}
		SetValues<FQuat>(*this, Metadata->PointRotation.Points, CurrentRotator.Quaternion());
	}
}

FText FCineSplineMetadataDetails::GetInterpType() const
{
	if (InterpTypeValue.IsSet())
	{
		const UEnum* InterpTypeEnum = StaticEnum<ECineSplineMetadataTangentType>();
		return InterpTypeEnum->GetDisplayNameTextByValue((int64)InterpTypeValue.GetValue());
	}
	return LOCTEXT("MultipleTypes", "Multiple Types");
}

void FCineSplineMetadataDetails::OnInterpTypeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (UCineSplineMetadata* Metadata = GetMetadata())
	{
		const UEnum* InterpTypeEnum = StaticEnum<ECineSplineMetadataTangentType>();
		const int64 InterpType =InterpTypeEnum->GetValueByNameString(*NewValue);
		if (InterpType != INDEX_NONE)
		{
			EInterpCurveMode Mode = ConvertCineSplineMetadataTangentTypeToInterpCurveMode(static_cast<ECineSplineMetadataTangentType>(InterpType));

			const FScopedTransaction Transaction(LOCTEXT("SetSplineMetadataInterpType", "Set spline metadata interp type"));
			Metadata->Modify();
			for (int32 Index : SelectedKeys)
			{
				Metadata->SetTangentType(Index, static_cast<ECineSplineMetadataTangentType>(InterpType));
			}
			Metadata->AutoSetTangents();

			if (GEditor)
			{
				GEditor->RedrawLevelEditingViewports(true);
			}
		}
	}
}

void FCineSplineMetadataDetails::UpdateInterpTypes()
{
	const UEnum* InterpTypeEnum = StaticEnum<ECineSplineMetadataTangentType>();
	InterpTypes.Reset();
	for (int32 Index = 0; Index < InterpTypeEnum->NumEnums() - 1; ++Index)
	{
		InterpTypes.Add(MakeShareable(new FString(InterpTypeEnum->GetNameStringByIndex(Index))));
	}
}

TSharedRef<SWidget> FCineSplineMetadataDetails::OnGenerateInterpTypeComboWidget(TSharedPtr<FString> InComboString)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InComboString))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FCineSplineMetadataDetails::OnInterpTypeComboWidgetOpened()
{
	TSharedPtr<FString> SelectedItem;
	for (TSharedPtr<FString> Item : InterpTypes)
	{
		if (Item->Equals(GetInterpType().ToString()))
		{
			SelectedItem = Item;
		}
	}
	InterpTypeComboBox->SetSelectedItem(SelectedItem);
}

void FCineSplineMetadataDetails::GenerateChildContent(IDetailGroup& DetailGroup)
{
	DetailGroup.AddWidgetRow()
		.ShouldAutoExpand(true)
		.RowTag("AbsolutePosition")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AbsolutePosition", "Absolute Position"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FCineSplineMetadataDetails::GetAbsolutePosition)
			.AllowSpin(true)
			.MinValue(TOptional<float>())
			.MaxValue(TOptional<float>())
			.MinSliderValue(TOptional<float>())
			.MaxSliderValue(TOptional<float>())
			.OnBeginSliderMovement(this, &FCineSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FCineSplineMetadataDetails::OnEndSliderMovement)
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FCineSplineMetadataDetails::OnSetAbsolutePosition)
			.OnValueChanged(this, &FCineSplineMetadataDetails::OnSetAbsolutePosition, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("PointRotation")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PointRotation", "Point Rotation"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SRotatorInputBox)
			.Roll(this, &FCineSplineMetadataDetails::GetRotation, EAxis::X)
			.Pitch(this, &FCineSplineMetadataDetails::GetRotation, EAxis::Y)
			.Yaw(this, &FCineSplineMetadataDetails::GetRotation, EAxis::Z)
			.OnRollChanged(this, &FCineSplineMetadataDetails::OnSetRotation, ETextCommit::Default, EAxis::X)
			.OnPitchChanged(this, &FCineSplineMetadataDetails::OnSetRotation, ETextCommit::Default, EAxis::Y)
			.OnYawChanged(this, &FCineSplineMetadataDetails::OnSetRotation, ETextCommit::Default, EAxis::Z)
			.OnRollCommitted(this, &FCineSplineMetadataDetails::OnSetRotation, EAxis::X)
			.OnPitchCommitted(this, &FCineSplineMetadataDetails::OnSetRotation, EAxis::Y)
			.OnYawCommitted(this, &FCineSplineMetadataDetails::OnSetRotation, EAxis::Z)
			.AllowSpin(true)
			.bColorAxisLabels(false)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("FocalLength")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FocalLength", "Focal Length"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FCineSplineMetadataDetails::GetFocalLength)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FCineSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FCineSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(0.0f)
			// Because we have no upper limit in MaxSliderValue, we need to "unspecify" the max value here, otherwise the spinner has a limited range,
			//  with TNumericLimits<NumericType>::Max() as the MaxValue and the spinning increment is huge
			.MaxValue(TOptional<float>())
			.MinSliderValue(0.0f)
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FCineSplineMetadataDetails::OnSetFocalLength)
			.OnValueChanged(this, &FCineSplineMetadataDetails::OnSetFocalLength, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("Aperture")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Aperture", "Aperture"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FCineSplineMetadataDetails::GetAperture)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FCineSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FCineSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(TOptional<float>())
			.MaxValue(TOptional<float>())
			.MinSliderValue(TOptional<float>()) // No lower limit
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FCineSplineMetadataDetails::OnSetAperture)
			.OnValueChanged(this, &FCineSplineMetadataDetails::OnSetAperture, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("FocusDistance")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FocusDistance", "Focus Distance"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FCineSplineMetadataDetails::GetFocusDistance)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FCineSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FCineSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(TOptional<float>())
			.MaxValue(TOptional<float>())
			.MinSliderValue(TOptional<float>()) // No lower limit
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FCineSplineMetadataDetails::OnSetFocusDistance)
			.OnValueChanged(this, &FCineSplineMetadataDetails::OnSetFocusDistance, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	UpdateInterpTypes();

	DetailGroup.AddWidgetRow()
		.RowTag("InterpType")
		.Visibility(TAttribute<EVisibility>(this, &FCineSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("InterpType", "Interp Type"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SAssignNew(InterpTypeComboBox, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&InterpTypes)
				.OnGenerateWidget(this, &FCineSplineMetadataDetails::OnGenerateInterpTypeComboWidget)
				.OnSelectionChanged(this, &FCineSplineMetadataDetails::OnInterpTypeChanged)
				.OnComboBoxOpening(this, &FCineSplineMetadataDetails::OnInterpTypeComboWidgetOpened)
				[
					SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &FCineSplineMetadataDetails::GetInterpType)
				]
		];
}

#undef LOCTEXT_NAMESPACE