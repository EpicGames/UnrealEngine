// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Visibility.h"
#include "Types/SlateEnums.h"
#include "ScopedTransaction.h"
#include "CineSplineMetadata.h"
#include "Widgets/Input/SComboBox.h"

#include "SplineMetadataDetailsFactory.h"

#include "CineSplineMetadataDetails.generated.h"

class UCineSplineMetadata;

UCLASS()
class UCineSplineMetadataDetailsFactory : public USplineMetadataDetailsFactoryBase
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<ISplineMetadataDetails> Create() override;
	virtual UClass* GetMetadataClass() const override;
};

class FCineSplineMetadataDetails : public ISplineMetadataDetails, public TSharedFromThis<FCineSplineMetadataDetails>
{
public:
	virtual ~FCineSplineMetadataDetails() = default;
	virtual FName GetName() const override { return FName("CineSplineMetadataDetails"); }
	virtual FText GetDisplayName() const override;
	virtual void Update(USplineComponent* InSplineComponent, const TSet<int32>& InSelectedKeys) override;
	virtual void GenerateChildContent(IDetailGroup& InGroup) override;

	TOptional<float> AbsolutePositionValue;
	TOptional<float> FocalLengthValue;
	TOptional<float> ApertureValue;
	TOptional<float> FocusDistanceValue;
	TOptional<FQuat> PointRotationValue;
	TOptional<ECineSplineMetadataTangentType> InterpTypeValue;

	USplineComponent* SplineComp = nullptr;
	TSet<int32> SelectedKeys;
	int32 SliderEnterCount = 0;

private:
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue);

	EVisibility IsEnabled() const { return (SelectedKeys.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	bool IsOnePointSelected() const { return SelectedKeys.Num() == 1; }

	TOptional<float> GetAbsolutePosition() const { return AbsolutePositionValue; }
	void OnSetAbsolutePosition(float NewValue, ETextCommit::Type CommitInfo);

	TOptional<float> GetFocalLength() const { return FocalLengthValue; }
	void OnSetFocalLength(float NewValue, ETextCommit::Type CommitInfo);

	TOptional<float> GetAperture() const { return ApertureValue; }
	void OnSetAperture(float NewValue, ETextCommit::Type CommitInfo);

	TOptional<float> GetFocusDistance() const { return FocusDistanceValue; }
	void OnSetFocusDistance(float NewValue, ETextCommit::Type CommitInfo);

	TOptional<float> GetRotation(EAxis::Type Axis) const;
	void OnSetRotation(float NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);

	FText GetInterpType() const;
	void OnInterpTypeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	TSharedRef<SWidget> OnGenerateInterpTypeComboWidget(TSharedPtr<FString> InComboString);
	void OnInterpTypeComboWidgetOpened();
	void UpdateInterpTypes();

	UCineSplineMetadata* GetMetadata() const;

private:
	TUniquePtr<FScopedTransaction> EditSliderValueTransaction;

	TArray<TSharedPtr<FString>> InterpTypes;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> InterpTypeComboBox;
};
